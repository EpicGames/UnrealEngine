// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBagDetails.h"

#include "DataHierarchyViewModelBase.h"
#include "StructUtilsEditorUtilsPrivate.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "PropertyBagDragDropHandler.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyPath.h"
#include "ScopedTransaction.h"
#include "SDraggableBox.h" // Custom widget for drag and drop sections
#include "StructUtilsEditorModule.h"
#include "StructUtilsMetadata.h"
#include "STypeSelector.h" // Custom widget for pill type selector
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HierarchyEditor/PropertyBagHierarchyViewModel.h"
#include "HierarchyEditor/SPropertyBagHierarchyEditor.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "StructUtils/UserDefinedStruct.h"
#include "UObject/EnumProperty.h"
#include "Layout/Visibility.h"

#include "Widgets/SDataHierarchyEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyBagDetails)

#define LOCTEXT_NAMESPACE "StructUtilsEditor"

////////////////////////////////////

TWeakPtr<SPropertyBagHierarchyEditor> FPropertyBagDetails::PropertyBagHierarchyEditor;
TWeakPtr<SWindow> FPropertyBagDetails::PropertyBagHierarchyEditorWindow;

namespace UE::StructUtils
{
/** Sets property descriptor based on a Blueprint pin type. */
void SetPropertyDescFromPin(FPropertyBagPropertyDesc& Desc, const FEdGraphPinType& PinType)
{
	const UPropertyBagSchema* Schema = GetDefault<UPropertyBagSchema>();
	check(Schema);

	// remove any existing containers
	Desc.ContainerTypes.Reset();

	// Fill Container types, if any
	switch (PinType.ContainerType)
	{
	case EPinContainerType::Array:
		Desc.ContainerTypes.Add(EPropertyBagContainerType::Array);
		break;
	case EPinContainerType::Set:
		Desc.ContainerTypes.Add(EPropertyBagContainerType::Set);
		break;
	case EPinContainerType::Map:
		Desc.ContainerTypes.Add(EPropertyBagContainerType::Map);
		break;
	default:
		break;
	}
	
	auto SetDescTypeFromPinType = [](
		const FName& Category,
		const FName& SubCategory,
		const TWeakObjectPtr<UObject>& SubCategoryObject,
		EPropertyBagPropertyType& OutType,
		TObjectPtr<const UObject>& OutTypeObject)
	{
		if (Category == UEdGraphSchema_K2::PC_Boolean)
		{
			OutType = EPropertyBagPropertyType::Bool;
			OutTypeObject = nullptr;
		}
		else if (Category == UEdGraphSchema_K2::PC_Byte)
		{
			if (UEnum* Enum = Cast<UEnum>(SubCategoryObject))
			{
				OutType = EPropertyBagPropertyType::Enum;
				OutTypeObject = SubCategoryObject.Get();
			}
			else
			{
				OutType = EPropertyBagPropertyType::Byte;
				OutTypeObject = nullptr;
			}
		}
		else if (Category == UEdGraphSchema_K2::PC_Int)
		{
			OutType = EPropertyBagPropertyType::Int32;
			OutTypeObject = nullptr;
		}
		else if (Category == UEdGraphSchema_K2::PC_Int64)
		{
			OutType = EPropertyBagPropertyType::Int64;
			OutTypeObject = nullptr;
		}
		else if (Category == UEdGraphSchema_K2::PC_Real)
		{
			if (SubCategory == UEdGraphSchema_K2::PC_Float)
			{
				OutType = EPropertyBagPropertyType::Float;
				OutTypeObject = nullptr;
			}
			else if (SubCategory == UEdGraphSchema_K2::PC_Double)
			{
				OutType = EPropertyBagPropertyType::Double;
				OutTypeObject = nullptr;
			}
		}
		else if (Category == UEdGraphSchema_K2::PC_Name)
		{
			OutType = EPropertyBagPropertyType::Name;
			OutTypeObject = nullptr;
		}
		else if (Category == UEdGraphSchema_K2::PC_String)
		{
			OutType = EPropertyBagPropertyType::String;
			OutTypeObject = nullptr;
		}
		else if (Category == UEdGraphSchema_K2::PC_Text)
		{
			OutType = EPropertyBagPropertyType::Text;
			OutTypeObject = nullptr;
		}
		else if (Category == UEdGraphSchema_K2::PC_Enum)
		{
			OutType = EPropertyBagPropertyType::Enum;
			OutTypeObject = SubCategoryObject.Get();
		}
		else if (Category == UEdGraphSchema_K2::PC_Struct)
		{
			OutType = EPropertyBagPropertyType::Struct;
			OutTypeObject = SubCategoryObject.Get();
		}
		else if (Category == UEdGraphSchema_K2::PC_Object)
		{
			OutType = EPropertyBagPropertyType::Object;
			OutTypeObject = SubCategoryObject.Get();
		}
		else if (Category == UEdGraphSchema_K2::PC_SoftObject)
		{
			OutType = EPropertyBagPropertyType::SoftObject;
			OutTypeObject = SubCategoryObject.Get();
		}
		else if (Category == UEdGraphSchema_K2::PC_Class)
		{
			OutType = EPropertyBagPropertyType::Class;
			OutTypeObject = SubCategoryObject.Get();
		}
		else if (Category == UEdGraphSchema_K2::PC_SoftClass)
		{
			OutType = EPropertyBagPropertyType::SoftClass;
			OutTypeObject = SubCategoryObject.Get();
		}
		else
		{
			ensureMsgf(false, TEXT("Unhandled pin category %s"), *Category.ToString());
		}
	};

	if (PinType.ContainerType == EPinContainerType::Map)
	{
		SetDescTypeFromPinType(PinType.PinCategory, PinType.PinSubCategory, PinType.PinSubCategoryObject, Desc.KeyType, Desc.KeyTypeObject);
		
		if (PinType.PinValueType.TerminalCategory == NAME_None)
		{
			SetDescTypeFromPinType(PinType.PinCategory, PinType.PinSubCategory, PinType.PinSubCategoryObject, Desc.ValueType, Desc.ValueTypeObject);
		}
		else
		{
			SetDescTypeFromPinType(PinType.PinValueType.TerminalCategory, PinType.PinValueType.TerminalSubCategory, PinType.PinValueType.TerminalSubCategoryObject, Desc.ValueType, Desc.ValueTypeObject);
		}
	}
	else
	{
		SetDescTypeFromPinType(PinType.PinCategory, PinType.PinSubCategory, PinType.PinSubCategoryObject, Desc.ValueType, Desc.ValueTypeObject);
	}
}

/** @return Blueprint pin type from property descriptor. */
FEdGraphPinType GetPropertyDescAsPin(const FPropertyBagPropertyDesc& Desc)
{
	UEnum* PropertyTypeEnum = StaticEnum<EPropertyBagPropertyType>();
	check(PropertyTypeEnum);
	const UPropertyBagSchema* Schema = GetDefault<UPropertyBagSchema>();
	check(Schema);

	FEdGraphPinType PinType;
	PinType.PinSubCategory = NAME_None;

	// Container type
	//@todo: Handle nested containers in property selection.
	const EPropertyBagContainerType ContainerType = Desc.ContainerTypes.GetFirstContainerType();
	switch (ContainerType)
	{
	case EPropertyBagContainerType::Array:
		PinType.ContainerType = EPinContainerType::Array;
		break;
	case EPropertyBagContainerType::Set:
		PinType.ContainerType = EPinContainerType::Set;
		break;
	case EPropertyBagContainerType::Map:
		PinType.ContainerType = EPinContainerType::Map;
		break;
	default:
		PinType.ContainerType = EPinContainerType::None;
	}

	auto SetPinTypeFromDescType = [](
		const EPropertyBagPropertyType& Type,
		const TObjectPtr<const UObject>& TypeObject,
		FName& OutCategory,
		FName& OutSubCategory,
		TWeakObjectPtr<UObject>& OutSubCategoryObject)
	{
		switch (Type)
		{
		case EPropertyBagPropertyType::Bool:
			OutCategory = UEdGraphSchema_K2::PC_Boolean;
			break;
		case EPropertyBagPropertyType::Byte:
			OutCategory = UEdGraphSchema_K2::PC_Byte;
			break;
		case EPropertyBagPropertyType::Int32:
			OutCategory = UEdGraphSchema_K2::PC_Int;
			break;
		case EPropertyBagPropertyType::Int64:
			OutCategory = UEdGraphSchema_K2::PC_Int64;
			break;
		case EPropertyBagPropertyType::Float:
			OutCategory = UEdGraphSchema_K2::PC_Real;
			OutSubCategory = UEdGraphSchema_K2::PC_Float;
			break;
		case EPropertyBagPropertyType::Double:
			OutCategory = UEdGraphSchema_K2::PC_Real;
			OutSubCategory = UEdGraphSchema_K2::PC_Double;
			break;
		case EPropertyBagPropertyType::Name:
			OutCategory = UEdGraphSchema_K2::PC_Name;
			break;
		case EPropertyBagPropertyType::String:
			OutCategory = UEdGraphSchema_K2::PC_String;
			break;
		case EPropertyBagPropertyType::Text:
			OutCategory = UEdGraphSchema_K2::PC_Text;
			break;
		case EPropertyBagPropertyType::Enum:
			//Always use the byte type. The compiler will generate the proper FByteProperty or FEnumProperty
			OutCategory = UEdGraphSchema_K2::PC_Byte;
			OutSubCategoryObject = const_cast<UObject*>(TypeObject.Get());
			break;
		case EPropertyBagPropertyType::Struct:
			OutCategory = UEdGraphSchema_K2::PC_Struct;
			OutSubCategoryObject = const_cast<UObject*>(TypeObject.Get());
			break;
		case EPropertyBagPropertyType::Object:
			OutCategory = UEdGraphSchema_K2::PC_Object;
			OutSubCategoryObject = const_cast<UObject*>(TypeObject.Get());
			break;
		case EPropertyBagPropertyType::SoftObject:
			OutCategory = UEdGraphSchema_K2::PC_SoftObject;
			OutSubCategoryObject = const_cast<UObject*>(TypeObject.Get());
			break;
		case EPropertyBagPropertyType::Class:
			OutCategory = UEdGraphSchema_K2::PC_Class;
			OutSubCategoryObject = const_cast<UObject*>(TypeObject.Get());
			break;
		case EPropertyBagPropertyType::SoftClass:
			OutCategory = UEdGraphSchema_K2::PC_SoftClass;
			OutSubCategoryObject = const_cast<UObject*>(TypeObject.Get());
			break;
		case EPropertyBagPropertyType::Int8:	// Warning : Type only partially supported (Blueprint does not support the int8 type)
			OutCategory = UEdGraphSchema_K2::PC_Int;
			break;
		case EPropertyBagPropertyType::Int16:	// Warning : Type only partially supported (Blueprint does not support the int16 type)
			OutCategory = UEdGraphSchema_K2::PC_Int;
			break;
		case EPropertyBagPropertyType::UInt16:	// Warning : Type only partially supported (Blueprint does not support the uint16 type)
			OutCategory = UEdGraphSchema_K2::PC_Int;
			break;
		case EPropertyBagPropertyType::UInt32:	// Warning : Type only partially supported (Blueprint does not support the uint32 type)
			OutCategory = UEdGraphSchema_K2::PC_Int;
			break;
		case EPropertyBagPropertyType::UInt64:	// Warning : Type only partially supported (Blueprint does not support the uint64 type)
			OutCategory = UEdGraphSchema_K2::PC_Int64;
			break;
		default:
			ensureMsgf(false, TEXT("Unhandled value type %s"), *UEnum::GetValueAsString(Type));
			break;
		}
	};

	if (PinType.ContainerType == EPinContainerType::Map)
	{
		SetPinTypeFromDescType(Desc.KeyType, Desc.KeyTypeObject, PinType.PinCategory, PinType.PinSubCategory, PinType.PinSubCategoryObject);
		SetPinTypeFromDescType(Desc.ValueType, Desc.ValueTypeObject, PinType.PinValueType.TerminalCategory, PinType.PinValueType.TerminalSubCategory, PinType.PinValueType.TerminalSubCategoryObject);
	}
	else
	{
		SetPinTypeFromDescType(Desc.ValueType, Desc.ValueTypeObject, PinType.PinCategory, PinType.PinSubCategory, PinType.PinSubCategoryObject);
	}

	return PinType;
}

bool CanCreateValidGraphPinTypeForPropertyDesc(const FPropertyBagPropertyDesc& Desc)
{
	auto IsValidType = [](EPropertyBagPropertyType Type, const TObjectPtr<const UObject>& TypeObject)
		{
			switch (Type)
			{
			case EPropertyBagPropertyType::Int8:
			case EPropertyBagPropertyType::Int16:
			case EPropertyBagPropertyType::UInt16:
			case EPropertyBagPropertyType::UInt32:
				return false;
			case EPropertyBagPropertyType::Enum:
			case EPropertyBagPropertyType::Struct:
			case EPropertyBagPropertyType::Object:
			case EPropertyBagPropertyType::SoftObject:
			case EPropertyBagPropertyType::Class:
			case EPropertyBagPropertyType::SoftClass:
				return TypeObject != nullptr;
			default:
				return true;
			}
		};

	if (Desc.ContainerTypes.Num() > 1)
	{
		return false;
	}

	if (Desc.KeyType != EPropertyBagPropertyType::None && !IsValidType(Desc.KeyType, Desc.KeyTypeObject))
	{
		return false;
	}

	return IsValidType(Desc.ValueType, Desc.ValueTypeObject);
}

namespace Private
{
/** @return true if the property is one of the known missing types. */
bool HasMissingType(const TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	if (!PropertyHandle)
	{
		return false;
	}

	// Handles Struct
	if (FStructProperty* StructProperty = CastField<FStructProperty>(PropertyHandle->GetProperty()))
	{
		return StructProperty->Struct == FPropertyBagMissingStruct::StaticStruct();
	}
	// Handles Object, SoftObject, Class, SoftClass.
	if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(PropertyHandle->GetProperty()))
	{
		return ObjectProperty->PropertyClass == UPropertyBagMissingObject::StaticClass();
	}
	// Handles Enum
	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(PropertyHandle->GetProperty()))
	{
		return EnumProperty->GetEnum() == StaticEnum<EPropertyBagMissingEnum>();
	}

	return false;
}

/** @return property descriptors of the property bag struct common to all edited properties. */
TArray<FPropertyBagPropertyDesc> GetCommonPropertyDescs(const TSharedPtr<IPropertyHandle>& StructProperty)
{
	TArray<FPropertyBagPropertyDesc> PropertyDescs;

	if (const UPropertyBag* BagStruct = GetCommonBagStruct(StructProperty))
	{
		PropertyDescs = BagStruct->GetPropertyDescs();
	}

	return PropertyDescs;
}

/** Creates new property bag struct and sets all properties to use it, migrating over old values. */
void SetPropertyDescs(const TSharedPtr<IPropertyHandle>& StructProperty, const TConstArrayView<FPropertyBagPropertyDesc> PropertyDescs)
{
	if (ensure(IsScriptStruct<FInstancedPropertyBag>(StructProperty)))
	{
		// Create new bag struct
		const UPropertyBag* NewBagStruct = UPropertyBag::GetOrCreateFromDescs(PropertyDescs);

		// Migrate structs to the new type, copying values over.
		StructProperty->EnumerateRawData([&NewBagStruct](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
		{
			if (RawData)
			{
				if (FInstancedPropertyBag* Bag = static_cast<FInstancedPropertyBag*>(RawData))
				{
					Bag->MigrateToNewBagStruct(NewBagStruct);
				}
			}

			return true;
		});
	}
}

FName GetPropertyNameSafe(const TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	const FProperty* Property = PropertyHandle ? PropertyHandle->GetProperty() : nullptr;
	if (Property != nullptr)
	{
		return Property->GetFName();
	}
	return FName();
}

/** @return true of the property name is not used yet by the property bag structure common to all edited properties. */
bool IsUniqueName(const FName NewName, const FName OldName, const TSharedPtr<IPropertyHandle>& StructProperty)
{
	if (NewName == OldName)
	{
		return false;
	}

	if (!StructProperty || !StructProperty->IsValidHandle())
	{
		return false;
	}

	bool bFound = false;

	if (ensure(IsScriptStruct<FInstancedPropertyBag>(StructProperty)))
	{
		StructProperty->EnumerateConstRawData([&bFound, NewName](const void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
		{
			if (const FInstancedPropertyBag* Bag = static_cast<const FInstancedPropertyBag*>(RawData))
			{
				if (const UPropertyBag* BagStruct = Bag->GetPropertyBagStruct())
				{
					const bool bContains = BagStruct->GetPropertyDescs().ContainsByPredicate([NewName](const FPropertyBagPropertyDesc& Desc)
					{
						return Desc.Name == NewName;
					});
					if (bContains)
					{
						bFound = true;
						return false; // Stop iterating
					}
				}
			}

			return true;
		});
	}

	return !bFound;
}

bool CanHaveMemberVariableOfType(const FEdGraphPinType& PinType)
{
	if (PinType.PinCategory == UEdGraphSchema_K2::PC_Exec
		|| PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard
		|| PinType.PinCategory == UEdGraphSchema_K2::PC_MCDelegate
		|| PinType.PinCategory == UEdGraphSchema_K2::PC_Delegate
		|| PinType.PinCategory == UEdGraphSchema_K2::PC_Interface)
	{
		return false;
	}

	return true;
}

FText GetAccessSpecifierNameFromFlags(const EPropertyFlags Flags)
{
	// TODO: Support 'protected'. For now treat protected and private the same.
	if (!!(Flags & (CPF_NativeAccessSpecifierPrivate | CPF_NativeAccessSpecifierProtected)))
	{
		return LOCTEXT("AccessSpecifierPrivate", "Private");
	}
	else // Public flag or not, should be treated as public.
	{
		return LOCTEXT("AccessSpecifierPublic", "Public");
	}
}

/** Checks if the value for a source property in a source struct has the same value that the target property in the target struct. */
bool ArePropertiesIdentical(
	const FPropertyBagPropertyDesc* InSourcePropertyDesc,
	const FInstancedPropertyBag& InSourceInstance,
	const FPropertyBagPropertyDesc* InTargetPropertyDesc,
	const FInstancedPropertyBag& InTargetInstance)
{
	if (!InSourceInstance.IsValid()
		|| !InTargetInstance.IsValid()
		|| !InSourcePropertyDesc
		|| !InSourcePropertyDesc->CachedProperty
		|| !InTargetPropertyDesc
		|| !InTargetPropertyDesc->CachedProperty)
	{
		return false;
	}

	if (!InSourcePropertyDesc->CompatibleType(*InTargetPropertyDesc))
	{
		return false;
	}

	const uint8* SourceValueAddress = InSourceInstance.GetValue().GetMemory() + InSourcePropertyDesc->CachedProperty->GetOffset_ForInternal();
	const uint8* TargetValueAddress = InTargetInstance.GetValue().GetMemory() + InTargetPropertyDesc->CachedProperty->GetOffset_ForInternal();

	return InSourcePropertyDesc->CachedProperty->Identical(SourceValueAddress, TargetValueAddress);
}

/** Copy the value for a source property in a source struct to the target property in the target struct. */
void CopyPropertyValue(const FPropertyBagPropertyDesc* InSourcePropertyDesc, const FInstancedPropertyBag& InSourceInstance, const FPropertyBagPropertyDesc* InTargetPropertyDesc, FInstancedPropertyBag& InTargetInstance)
{
	if (!InSourceInstance.IsValid() || !InTargetInstance.IsValid() || !InSourcePropertyDesc || !InSourcePropertyDesc->CachedProperty || !InTargetPropertyDesc || !InTargetPropertyDesc->CachedProperty)
	{
		return;
	}

	// Can't copy if they are not compatible.
	if (!InSourcePropertyDesc->CompatibleType(*InTargetPropertyDesc))
	{
		return;
	}

	const uint8* SourceValueAddress = InSourceInstance.GetValue().GetMemory() + InSourcePropertyDesc->CachedProperty->GetOffset_ForInternal();
	uint8* TargetValueAddress = InTargetInstance.GetMutableValue().GetMemory() + InTargetPropertyDesc->CachedProperty->GetOffset_ForInternal();

	InSourcePropertyDesc->CachedProperty->CopyCompleteValue(TargetValueAddress, SourceValueAddress);
}

void GetFilteredVariableTypeTree(const TSharedPtr<IPropertyHandle>& BagStructProperty, TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>>& TypeTree, const ETypeTreeFilter TypeTreeFilter, const UPropertyBagSchema* Schema)
{
	// The type selector popup might outlive this details view, so bag struct property can be invalid here.
	if (!BagStructProperty || !BagStructProperty->IsValidHandle())
	{
		return;
	}

	DECLARE_DELEGATE_RetVal_TwoParams(bool, FIsPinTypeAccepted, FEdGraphPinType, bool);
	FIsPinTypeAccepted IsPinTypeAcceptedDelegate{};

	if (TOptional<FFindUserFunctionResult> Result = FindUserFunction(BagStructProperty, Metadata::IsPinTypeAcceptedName); Result.IsSet())
	{
		check(Result.GetValue().Function && Result.GetValue().Target);
		IsPinTypeAcceptedDelegate = FIsPinTypeAccepted::CreateUFunction(Result.GetValue().Target, Result.GetValue().Function->GetFName());
	}

	auto IsPinTypeAccepted = [&IsPinTypeAcceptedDelegate](const FEdGraphPinType& InPinType, bool bInIsChild) -> bool
	{
		if (IsPinTypeAcceptedDelegate.IsBound())
		{
			return IsPinTypeAcceptedDelegate.Execute(InPinType, bInIsChild);
		}
		else
		{
			return true;
		}
	};

	TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>> TempTypeTree;
	Schema->GetVariableTypeTree(TempTypeTree, TypeTreeFilter);

	// Filter
	for (TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& PinType : TempTypeTree)
	{
		if (!PinType.IsValid() || !IsPinTypeAccepted(PinType->GetPinType(/*bForceLoadSubCategoryObject*/false), /*bInIsChild=*/ false))
		{
			continue;
		}

		for (int32 ChildIndex = 0; ChildIndex < PinType->Children.Num();)
		{
			TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo> Child = PinType->Children[ChildIndex];
			if (Child.IsValid())
			{
				const FEdGraphPinType& ChildPinType = Child->GetPinType(/*bForceLoadSubCategoryObject*/false);

				if (!CanHaveMemberVariableOfType(ChildPinType) || !IsPinTypeAccepted(ChildPinType, /*bInIsChild=*/ true))
				{
					PinType->Children.RemoveAt(ChildIndex);
					continue;
				}
			}
			++ChildIndex;
		}

		TypeTree.Add(PinType);
	}
}

bool CanDeleteProperty(const TSharedPtr<IPropertyHandle>& InStructProperty, const TSharedPtr<IPropertyHandle>& ChildPropertyHandle)
{
	if (!InStructProperty || !InStructProperty->IsValidHandle() || !ChildPropertyHandle || !ChildPropertyHandle->IsValidHandle())
	{
		return false;
	}

	// Extra check provided by the user to cancel a remove action. Useful to provide the user a possibility to cancel the action if
	// the given property is in use elsewhere.
	if (TOptional<FFindUserFunctionResult> Result = FindUserFunction(InStructProperty, Metadata::CanRemovePropertyName); Result.IsSet())
	{
		check(Result.GetValue().Function && Result.GetValue().Target);

		FName PropertyName = ChildPropertyHandle->GetProperty()->GetFName();
		const UPropertyBag* PropertyBag = GetCommonBagStruct(InStructProperty);
		const FPropertyBagPropertyDesc* PropertyDesc = PropertyBag ? PropertyBag->FindPropertyDescByName(PropertyName) : nullptr;

		if (!PropertyDesc)
		{
			return false;
		}

		DECLARE_DELEGATE_RetVal_TwoParams(bool, FGetCanDeleteProperty, FGuid, FName);
		return FGetCanDeleteProperty::CreateUFunction(Result.GetValue().Target, Result.GetValue().Function->GetFName()).Execute(PropertyDesc->ID, PropertyDesc->Name);
	}
	else
	{
		return true;
	}
}

void DeleteProperty(TSharedPtr<IPropertyHandle> InStructProperty, TSharedPtr<IPropertyHandle> ChildPropertyHandle, const TSharedPtr<IPropertyUtilities>& PropUtils)
{
	if (!InStructProperty || !InStructProperty->IsValidHandle() || !ChildPropertyHandle || !ChildPropertyHandle->IsValidHandle())
	{
		return;
	}

	if (!CanDeleteProperty(InStructProperty, ChildPropertyHandle))
	{
		return;
	}

	ApplyChangesToPropertyDescs(
		FText::Format(LOCTEXT("OnPropertyDeleted", "Deleted property: {0}"), ChildPropertyHandle->GetPropertyDisplayName()),
		InStructProperty,
		[&ChildPropertyHandle](TArray<FPropertyBagPropertyDesc>& PropertyDescs)
		{
			const FProperty* Property = ChildPropertyHandle ? ChildPropertyHandle->GetProperty() : nullptr;
			PropertyDescs.RemoveAll([Property](const FPropertyBagPropertyDesc& Desc) { return Desc.CachedProperty == Property; });
		});
}

FEdGraphPinType GetPinInfo(const TSharedPtr<IPropertyHandle>& ChildPropertyHandle, const TSharedPtr<IPropertyHandle>& InBagStructProperty)
{
	// The SPinTypeSelector popup might outlive this details view, so bag struct property can be invalid here.
	if (!InBagStructProperty || !InBagStructProperty->IsValidHandle() || !ChildPropertyHandle || !ChildPropertyHandle->IsValidHandle())
	{
		return FEdGraphPinType();
	}

	TArray<FPropertyBagPropertyDesc> PropertyDescs = Private::GetCommonPropertyDescs(InBagStructProperty);

	const FProperty* Property = ChildPropertyHandle->GetProperty();
	if (FPropertyBagPropertyDesc* Desc = PropertyDescs.FindByPredicate([Property](const FPropertyBagPropertyDesc& Desc){ return Desc.CachedProperty == Property; }))
	{
		return GetPropertyDescAsPin(*Desc);
	}

	return FEdGraphPinType();
}

void PinInfoChanged(TSharedPtr<IPropertyHandle> ChildPropertyHandle, const TSharedPtr<IPropertyHandle>& InBagStructProperty, const TSharedPtr<IPropertyUtilities>& InPropUtils, const FEdGraphPinType& PinType)
{
	// The SPinTypeSelector popup might outlive this details view, so bag struct property can be invalid here.
	if (!InBagStructProperty || !InBagStructProperty->IsValidHandle() || !ChildPropertyHandle || !ChildPropertyHandle->IsValidHandle())
	{
		return;
	}

	ApplyChangesToPropertyDescs(
		FText::Format(LOCTEXT("OnPropertyTypeChanged", "Changed property type: {0}"), ChildPropertyHandle->GetPropertyDisplayName()),
		InBagStructProperty,
		[&PinType, &ChildPropertyHandle](TArray<FPropertyBagPropertyDesc>& PropertyDescs)
		{
			// Find and change struct type
			const FProperty* Property = ChildPropertyHandle ? ChildPropertyHandle->GetProperty() : nullptr;
			if (FPropertyBagPropertyDesc* Desc = PropertyDescs.FindByPredicate([Property](const FPropertyBagPropertyDesc& Desc){ return Desc.CachedProperty == Property; }))
			{
				SetPropertyDescFromPin(*Desc, PinType);
			}
		});
}

FOptionalSize GetDesiredInputWidgetSizeByType(const TSharedPtr<IPropertyHandle>& ChildPropertyHandle)
{
	static constexpr float StringPropertyDesiredSize = 120.f;
	static constexpr float NamePropertyDesiredSize = 50.f;

	if (const FProperty* Property = ChildPropertyHandle ? ChildPropertyHandle->GetProperty() : nullptr)
	{
		if (Property->IsA(FStrProperty::StaticClass()))
		{
			return StringPropertyDesiredSize;
		}
		else if (Property->IsA(FNameProperty::StaticClass()))
		{
			return NamePropertyDesiredSize;
		}
	}

	return {};
}
} // UE::StructUtils::Private

TSharedRef<SWidget> CreateTypeSelectionWidget(TSharedPtr<IPropertyHandle> ChildPropertyHandle, const TSharedPtr<IPropertyHandle>& InBagStructProperty, const TSharedPtr<IPropertyUtilities>& InPropUtils, SPinTypeSelector::ESelectorType SelectorType, const bool bAllowContainers, TSubclassOf<UEdGraphSchema> PropertyBagSchemaClass)
{
	const UPropertyBagSchema* Schema = PropertyBagSchemaClass ? GetDefault<UPropertyBagSchema>(PropertyBagSchemaClass) : GetDefault<UPropertyBagSchema>();

	return SNew(SBox)
		.HAlign(HAlign_Right)
		.Padding(FMargin(4, 0))
		[
			SNew(STypeSelector, FGetPinTypeTree::CreateLambda(
				[BagStructProperty = InBagStructProperty, SchemaWeak = TWeakObjectPtr<const UPropertyBagSchema>(Schema)]
				(TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>>& TypeTree, const ETypeTreeFilter TypeTreeFilter)
				{
					if (const UPropertyBagSchema* Schema = SchemaWeak.Get())
					{
						UE::StructUtils::Private::GetFilteredVariableTypeTree(BagStructProperty, TypeTree, TypeTreeFilter, Schema);
					}
				}))
			.TargetPinType_Lambda([ChildPropertyHandle = ChildPropertyHandle, BagStructProperty = InBagStructProperty]()
			{
				return Private::GetPinInfo(ChildPropertyHandle, BagStructProperty);
			})
			.OnPinTypeChanged_Lambda([ChildPropertyHandle = ChildPropertyHandle, BagStructProperty = InBagStructProperty, PropUtils = InPropUtils](const FEdGraphPinType& PinType)
			{
				return Private::PinInfoChanged(ChildPropertyHandle, BagStructProperty, PropUtils, PinType);
			})
			.Schema(Schema)
			.bAllowContainers(bAllowContainers)
			.SelectorType(SelectorType)
			.TypeTreeFilter(ETypeTreeFilter::None)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
}

const UPropertyBag* GetCommonBagStruct(TSharedPtr<IPropertyHandle> StructProperty)
{
	const UPropertyBag* CommonBagStruct = nullptr;

	if (ensure(Private::IsScriptStruct<FInstancedPropertyBag>(StructProperty)))
	{
		StructProperty->EnumerateConstRawData([&CommonBagStruct](const void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
		{
			if (RawData)
			{
				const FInstancedPropertyBag* Bag = static_cast<const FInstancedPropertyBag*>(RawData);

				const UPropertyBag* BagStruct = Bag->GetPropertyBagStruct();
				if (CommonBagStruct && CommonBagStruct != BagStruct)
				{
					// Multiple struct types on the sources - show nothing set
					CommonBagStruct = nullptr;
					return false;
				}
				CommonBagStruct = BagStruct;
			}

			return true;
		});
	}

	return CommonBagStruct;
}

TSubclassOf<UPropertyBagSchema> ExtractPropertyBagSchemaClass(TSharedRef<IPropertyHandle> InPropertyBagHandle)
{
	FString PropertyBagSchemaClassName;
	
	/** We first try to extract the instance metadata, if specified. */
	if (const FString* InstancePropertyBagSchemaClassName = InPropertyBagHandle->GetInstanceMetaData(UE::StructUtils::Metadata::PropertyBagSchemaClassName))
	{
		PropertyBagSchemaClassName = *InstancePropertyBagSchemaClassName;
	}
	
	/** If not available, we attempt to use the class metadata. */
	if (PropertyBagSchemaClassName.IsEmpty())
	{
		if (FProperty* MetaDataProperty = InPropertyBagHandle->GetMetaDataProperty())
		{
			PropertyBagSchemaClassName = MetaDataProperty->GetMetaData(UE::StructUtils::Metadata::PropertyBagSchemaClassName);
		}
	}
	
	if (!PropertyBagSchemaClassName.IsEmpty())
	{
		return FindObject<UClass>(nullptr, *PropertyBagSchemaClassName);
	}
	
	return nullptr;
}

const UPropertyBagSchema* ExtractPropertyBagSchemaCDO(TSharedRef<IPropertyHandle> InPropertyBagHandle)
{
	TSubclassOf<UPropertyBagSchema> SchemaClass = ExtractPropertyBagSchemaClass(InPropertyBagHandle);
	
	if (const UPropertyBagSchema* Schema = SchemaClass ? GetDefault<UPropertyBagSchema>(SchemaClass) : GetDefault<UPropertyBagSchema>())
	{
		return Schema;
	}
	
	return nullptr;
}

UPropertyBagHierarchyRoot* ExtractHierarchyRoot(TSharedRef<IPropertyHandle> InPropertyBagHandle)
{
	if (!InPropertyBagHandle->IsValidHandle())
	{
		return nullptr;
	}
	
	if (const UPropertyBagSchema* Schema = ExtractPropertyBagSchemaCDO(InPropertyBagHandle))
	{
		TArray<UObject*> OuterObjects;
		InPropertyBagHandle->GetOuterObjects(OuterObjects);
		return Schema->GetHierarchyRoot(OuterObjects);
	}
	
	return nullptr;
}

void ApplyChangesToPropertyDescs(const FText& SessionName, const TSharedPtr<IPropertyHandle>& StructProperty, TFunctionRef<void(TArray<FPropertyBagPropertyDesc>&)> Function)
{
	if (!StructProperty || !StructProperty.Get()->IsValidHandle())
	{
		return;
	}

	FScopedTransaction Transaction(SessionName);
	TArray<FPropertyBagPropertyDesc> PropertyDescs = Private::GetCommonPropertyDescs(StructProperty);
	StructProperty->NotifyPreChange();
	
	if (UPropertyBagHierarchyRoot* Root = ExtractHierarchyRoot(StructProperty.ToSharedRef()))
	{
		Root->Modify();
	}
	
	Function(PropertyDescs);

	Private::SetPropertyDescs(StructProperty, PropertyDescs);

	StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
	StructProperty->NotifyFinishedChangingProperties();
	
	StructProperty->RequestRebuildChildren();
}

void ApplyChangesToPropertyDescs(const FText& SessionName, const TSharedPtr<IPropertyHandle>& StructProperty, const TSharedPtr<IPropertyUtilities>& PropUtils, TFunctionRef<void(TArray<FPropertyBagPropertyDesc>&)> Function)
{
	ApplyChangesToPropertyDescs(SessionName, StructProperty, Function);
}

void ApplyChangesToSinglePropertyDesc(const FText& SessionName, const TSharedPtr<IPropertyHandle> PropertyHandle, const TSharedPtr<IPropertyHandle>& StructProperty, TFunctionRef<void(FPropertyBagPropertyDesc&)> Function)
{
	ApplyChangesToPropertyDescs(SessionName, StructProperty, [Function = std::move(Function), Property = PropertyHandle->GetProperty()](TArray<FPropertyBagPropertyDesc>& PropertyDescs)
		{
			if (FPropertyBagPropertyDesc* Desc = PropertyDescs.FindByPredicate([Property](const FPropertyBagPropertyDesc& OutDesc) { return OutDesc.CachedProperty == Property; }))
			{
				Function(*Desc);
			}
		});
}

void ApplyChangesToSinglePropertyDesc(const FText& SessionName, const FPropertyBagPropertyDesc& PropertyDesc, const TSharedPtr<IPropertyHandle>& StructProperty, TFunctionRef<void(FPropertyBagPropertyDesc&)> Function)
{
	ApplyChangesToPropertyDescs(SessionName, StructProperty, [Function = std::move(Function), &PropertyDesc](TArray<FPropertyBagPropertyDesc>& PropertyDescs)
		{
			if (FPropertyBagPropertyDesc* Desc = PropertyDescs.FindByPredicate([&PropertyDesc](const FPropertyBagPropertyDesc& OutDesc) { return OutDesc == PropertyDesc; }))
			{
				Function(*Desc);
			}
		});
}

void ApplyChangesToSinglePropertyDesc(const FText& SessionName, const FPropertyBagPropertyDesc& PropertyDesc, const TSharedPtr<IPropertyHandle>& StructProperty, const TSharedPtr<IPropertyUtilities>& PropUtils, TFunctionRef<void(FPropertyBagPropertyDesc&)> Function)
{
	ApplyChangesToSinglePropertyDesc(SessionName, PropertyDesc, StructProperty, Function);
}

namespace Constants
{
	static constexpr int32 MaxCategoryLength = 70;
	static constexpr int32 MinSubMenuTextBoxWidth = 40;
	static constexpr int32 MaxSubMenuTextBoxWidth = 800;
	static constexpr int32 SubMenuTextBoxPadding = 5;

	// Special case for categories. Alphanumeric, but including spaces and `|` for nested categories.
	static constexpr TCHAR InvalidCategoryCharacters[] = TEXT("\"',/.:&!?~\\\n\r\t@#(){}[]<>=;^%$`*+-");
}
} // UE::StructUtils


/** Drag & drop handler for hierarchy categories in the details panel. Supports both dragging (source) and dropping (target).
 *  Uses FHierarchyDragDropOp for dragging (categories are not properties), and accepts both FHierarchyDragDropOp and
 *  FPropertyBagDetailsDragDropOp for drops (properties dragged onto categories use the latter). */
class FHierarchyCategoryDetailsDragDropHandler : public IDetailDragDropHandler
{
public:
	/** The delegate receives the full drag drop event so it can update decorators, plus the extracted element and drop zone. */
	DECLARE_DELEGATE_RetVal_ThreeParams(TOptional<EItemDropZone>, FCanAcceptHierarchyDrop, const FDragDropEvent& /*DragDropEvent*/, TSharedPtr<FHierarchyElementViewModel> /*DraggedElement*/, EItemDropZone /*DropZone*/);
	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnHierarchyDrop, TSharedPtr<FHierarchyElementViewModel> /*DroppedElement*/, EItemDropZone /*DropZone*/);

	FHierarchyCategoryDetailsDragDropHandler(TSharedPtr<FHierarchyElementViewModel> InCategoryVM)
		: CategoryVM(InCategoryVM)
	{
	}

	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation() const override
	{
		TSharedPtr<FHierarchyElementViewModel> PinnedVM = CategoryVM.Pin();
		if (!PinnedVM.IsValid())
		{
			return nullptr;
		}

		TSharedPtr<FHierarchyDragDropOp> DragOp = MakeShared<FHierarchyDragDropOp>(FDataHierarchyDragDropContext({FDraggedElementEntry{PinnedVM}}));
		DragOp->Construct();
		return DragOp;
	}

	virtual bool UseHandleWidget() const override { return true; }

	void BindCanAcceptDrop(FCanAcceptHierarchyDrop&& InDelegate) { CanAcceptDropDelegate = MoveTemp(InDelegate); }
	void BindOnDrop(FOnHierarchyDrop&& InDelegate) { OnDropDelegate = MoveTemp(InDelegate); }

protected:
	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropSource, EItemDropZone DropZone) const override
	{
		TSharedPtr<FHierarchyElementViewModel> DraggedElement = ExtractDraggedElement(DragDropSource);
		if (DraggedElement.IsValid() && CanAcceptDropDelegate.IsBound())
		{
			return CanAcceptDropDelegate.Execute(DragDropSource, DraggedElement, DropZone);
		}
		return TOptional<EItemDropZone>();
	}

	virtual bool AcceptDrop(const FDragDropEvent& DragDropSource, EItemDropZone DropZone) const override
	{
		TSharedPtr<FHierarchyElementViewModel> DraggedElement = ExtractDraggedElement(DragDropSource);
		if (DraggedElement.IsValid() && OnDropDelegate.IsBound())
		{
			return OnDropDelegate.Execute(DraggedElement, DropZone).IsEventHandled();
		}
		return false;
	}

private:
	/** Extracts the hierarchy element from either FPropertyBagDetailsDragDropOp or FHierarchyDragDropOp. */
	static TSharedPtr<FHierarchyElementViewModel> ExtractDraggedElement(const FDragDropEvent& DragDropEvent)
	{
		if (const TSharedPtr<FPropertyBagDetailsDragDropOp> PropertyOp = DragDropEvent.GetOperationAs<FPropertyBagDetailsDragDropOp>())
		{
			return PropertyOp->GetHierarchyElement();
		}
		if (const TSharedPtr<FHierarchyDragDropOp> HierarchyOp = DragDropEvent.GetOperationAs<FHierarchyDragDropOp>())
		{
			return HierarchyOp->GetDraggedElement().Pin();
		}
		return nullptr;
	}

	TWeakPtr<FHierarchyElementViewModel> CategoryVM;
	FCanAcceptHierarchyDrop CanAcceptDropDelegate;
	FOnHierarchyDrop OnDropDelegate;
};

/** The category builder will display all sub-categories & parameters contained within a given category in the details panel. */
class FHierarchyPropertyCategoryBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FHierarchyPropertyCategoryBuilder>
{
public:
	FHierarchyPropertyCategoryBuilder(const UPropertyBagHierarchyCategory& InHierarchyCategory, TSharedRef<FPropertyBagInstanceDataDetails> InOwningDetails);
	
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	
	virtual bool InitiallyCollapsed() const override;

	virtual FName GetName() const  override;

	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;

private:
	TWeakObjectPtr<const UPropertyBagHierarchyCategory> HierarchyCategory;
	FName CustomBuilderRowName;
	TWeakPtr<FPropertyBagInstanceDataDetails> OwningDetails;
};

FHierarchyPropertyCategoryBuilder::FHierarchyPropertyCategoryBuilder(const UPropertyBagHierarchyCategory& InHierarchyCategory, TSharedRef<FPropertyBagInstanceDataDetails> InOwningDetails)
	: HierarchyCategory(&InHierarchyCategory)
	, CustomBuilderRowName(HierarchyCategory->GetCategoryName())
	, OwningDetails(InOwningDetails)
{	
}

void FHierarchyPropertyCategoryBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	NodeRow.NameContent()
	[
		SNew(STextBlock)
		.Text_UObject(HierarchyCategory.Get(), &UHierarchyCategory::GetCategoryAsText)
		.ToolTipText_UObject(HierarchyCategory.Get(), &UHierarchyCategory::GetTooltip)
	];

	NodeRow.FilterString(FText::FromName(CustomBuilderRowName));

	// Add drag & drop support for categories via the hierarchy view model
	if (OwningDetails.IsValid() && HierarchyCategory.IsValid())
	{
		UPropertyBagHierarchyViewModel* HierarchyVM = OwningDetails.Pin()->GetHierarchyViewModel();
		if (HierarchyVM)
		{
			TSharedPtr<FHierarchyRootViewModel> RootVM = HierarchyVM->GetHierarchyRootViewModel();
			TSharedPtr<FHierarchyElementViewModel> CategoryVM = RootVM ? RootVM->FindViewModelForChild(HierarchyCategory.Get(), true) : nullptr;

			if (CategoryVM.IsValid())
			{
				TSharedPtr<FHierarchyCategoryDetailsDragDropHandler> DragDropHandler = MakeShared<FHierarchyCategoryDetailsDragDropHandler>(CategoryVM);

				DragDropHandler->BindCanAcceptDrop(
					FHierarchyCategoryDetailsDragDropHandler::FCanAcceptHierarchyDrop::CreateLambda(
						[CategoryVM](const FDragDropEvent& DragDropEvent, TSharedPtr<FHierarchyElementViewModel> DraggedElementVM, EItemDropZone DropZone) -> TOptional<EItemDropZone>
						{
							if (!DraggedElementVM.IsValid())
							{
								return TOptional<EItemDropZone>();
							}

							FHierarchyElementViewModel::FResultWithUserFeedback Result = CategoryVM->CanDropOn(DraggedElementVM, DropZone);

							// Update decorators on whichever op type is active
							if (const TSharedPtr<FPropertyBagDetailsDragDropOp> PropertyOp = DragDropEvent.GetOperationAs<FPropertyBagDetailsDragDropOp>())
							{
								TOptional<FPropertyBagDetailsDragDropOp::FDecoration> DecorationOverride;
								if (Result.UserFeedback.IsSet())
								{
									const FSlateBrush* Brush = FAppStyle::Get().GetBrush(Result.bResult ? "Graph.ConnectorFeedback.OK" : "Graph.ConnectorFeedback.Error");
									DecorationOverride.Emplace(Result.UserFeedback.GetValue(), Brush);
								}
								PropertyOp->SetDecoration(Result.bResult ? EPropertyBagDropState::Valid : EPropertyBagDropState::Invalid, MoveTemp(DecorationOverride));
							}
							else if (const TSharedPtr<FHierarchyDragDropOp> HierarchyOp = DragDropEvent.GetOperationAs<FHierarchyDragDropOp>())
							{
								HierarchyOp->SetDescription(Result.UserFeedback.IsSet() ? Result.UserFeedback.GetValue() : FText::GetEmpty());
							}

							return Result.bResult ? DropZone : TOptional<EItemDropZone>();
						}));

				DragDropHandler->BindOnDrop(
					FHierarchyCategoryDetailsDragDropHandler::FOnHierarchyDrop::CreateLambda(
						[CategoryVM](TSharedPtr<FHierarchyElementViewModel> DroppedElementVM, EItemDropZone DropZone) -> FReply
						{
							if (!DroppedElementVM.IsValid())
							{
								return FReply::Unhandled();
							}

							CategoryVM->OnDroppedOn(DroppedElementVM, DropZone);
							return FReply::Handled();
						}));

				NodeRow.DragDropHandler(MoveTemp(DragDropHandler));
			}
		}
	}
}

bool FHierarchyPropertyCategoryBuilder::InitiallyCollapsed() const
{
	return false;
}

FName FHierarchyPropertyCategoryBuilder::GetName() const
{
	return CustomBuilderRowName;
}

void FHierarchyPropertyCategoryBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (!OwningDetails.IsValid())
	{
		return;
	}
	
	TArray<const UPropertyBagHierarchyCategory*> ContainedCategories;
	HierarchyCategory->GetChildrenOfType(ContainedCategories, false);
	
	for (const UPropertyBagHierarchyCategory* ContainedHierarchyCategory : ContainedCategories)
	{
		// Skip categories that don't actually contain a property
		if (ContainedHierarchyCategory->DoesOneChildExist<UPropertyBagHierarchyProperty>(true))
		{
			ChildrenBuilder.AddCustomBuilder(MakeShared<FHierarchyPropertyCategoryBuilder>(*ContainedHierarchyCategory, OwningDetails.Pin().ToSharedRef()));
		}
	}
		
	TArray<const UPropertyBagHierarchyProperty*> ContainedProperties;
	HierarchyCategory->GetChildrenOfType<>(ContainedProperties, false);
	
	for (const UPropertyBagHierarchyProperty* ContainedHierarchyProperty : ContainedProperties)
	{
		TObjectKey<const UPropertyBagHierarchyProperty> Key(ContainedHierarchyProperty);
		if (const TSharedPtr<IPropertyHandle>* PropertyHandle = OwningDetails.Pin()->GetHierarchyPropertyHandleMap().Find(Key))
		{
			IDetailPropertyRow& PropertyRow = ChildrenBuilder.AddProperty(PropertyHandle->ToSharedRef());
			PropertyRow.Visibility(MakeAttributeSP(OwningDetails.Pin().Get(), &FPropertyBagInstanceDataDetails::ShouldShowHierarchyProperty, PropertyRow.GetPropertyHandle()));
			OwningDetails.Pin()->OnChildRowAdded(PropertyRow);
		}
	}
}

//----------------------------------------------------------------//
//  FPropertyBagInstanceDataDetails
//  - StructProperty is FInstancedPropertyBag
//  - ChildPropertyHandle a child property of the FInstancedPropertyBag::Value (FInstancedStruct)
//----------------------------------------------------------------//

/** Primary constructor. Values passed by parameter struct. */
FPropertyBagInstanceDataDetails::FPropertyBagInstanceDataDetails(const FPropertyBagInstanceDataDetails::FConstructParams& ConstructParams)
	: FInstancedStructDataDetails(ConstructParams.BagStructProperty.IsValid() ? ConstructParams.BagStructProperty->GetChildHandle(TEXT("Value")) : nullptr)
	, BagStructProperty(ConstructParams.BagStructProperty)
	, PropUtils(ConstructParams.PropUtils)
	, bAllowContainers(ConstructParams.bAllowContainers)
	, ChildRowFeatures(ConstructParams.ChildRowFeatures)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, bFixedLayout(false)
	, bAllowArrays(true)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	ensure(UE::StructUtils::Private::IsScriptStruct<FInstancedPropertyBag>(BagStructProperty));
	ensure(PropUtils != nullptr);

	if (BagStructProperty.IsValid() && BagStructProperty->IsValidHandle())
	{
		PropertyBagSchema = UE::StructUtils::ExtractPropertyBagSchemaCDO(BagStructProperty.ToSharedRef());

		HierarchyData.HierarchyRoot = UE::StructUtils::ExtractHierarchyRoot(BagStructProperty.ToSharedRef());
		if (HierarchyData.HierarchyRoot.IsValid())
		{
			HierarchyData.OnNavigateToHierarchyPropertyRequestedDelegate = ConstructParams.HierarchyParams.OnNavigateToHierarchyPropertyRequested;
			HierarchyData.HierarchyViewModel = ConstructParams.HierarchyParams.HierarchyViewModel;
		}
	}
}

/** For backwards compatibility. */
FPropertyBagInstanceDataDetails::FPropertyBagInstanceDataDetails(TSharedPtr<IPropertyHandle> InStructProperty, const TSharedPtr<IPropertyUtilities>& InPropUtils, const bool bInFixedLayout, const bool bInAllowContainers)
	: FInstancedStructDataDetails(InStructProperty.IsValid() ? InStructProperty->GetChildHandle(TEXT("Value")) : nullptr)
	, BagStructProperty(InStructProperty)
	, PropUtils(InPropUtils)
	, bAllowContainers(bInAllowContainers)
	, ChildRowFeatures(bInFixedLayout ? EPropertyBagChildRowFeatures::Fixed : EPropertyBagChildRowFeatures::Default)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	, bFixedLayout(bInFixedLayout) // For backwards compatibility
	, bAllowArrays(bInAllowContainers) // For backwards compatibility
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	ensure(UE::StructUtils::Private::IsScriptStruct<FInstancedPropertyBag>(BagStructProperty));
	ensure(PropUtils != nullptr);

	if (BagStructProperty.IsValid() && BagStructProperty->IsValidHandle())
	{
		PropertyBagSchema = UE::StructUtils::ExtractPropertyBagSchemaCDO(BagStructProperty.ToSharedRef());
		HierarchyData.HierarchyRoot = UE::StructUtils::ExtractHierarchyRoot(BagStructProperty.ToSharedRef());
	}
}

void FPropertyBagInstanceDataDetails::GenerateHierarchyShowAdvancedRow(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (!HierarchyData.HierarchyRoot.IsValid())
	{
		return;
	}
	
	// At last we add a row for the 'Show Advanced' button. Only displayed if there is at least one advanced property
	FDetailWidgetRow& AdvancedWidgetRow = ChildrenBuilder.AddCustomRow(INVTEXT("Advanced"));
	AdvancedWidgetRow.Visibility(TAttribute<EVisibility>::CreateSP(this, &FPropertyBagInstanceDataDetails::GetAdvancedButtonVisibility));
		
	AdvancedWidgetRow.WholeRowContent()
	.HAlign(HAlign_Center)
	[
		SAssignNew(ShowAdvancedButton, SButton)
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.ToolTipText(LOCTEXT("DisplayedAdvancedProperties", "Display advanced properties"))
		.OnClicked_Lambda([this]()
		{
			bShouldShowAdvanced = !bShouldShowAdvanced;
			return FReply::Handled();
		})
		[
			SNew(SImage)
			.Image(this, &FPropertyBagInstanceDataDetails::GetAdvancedImage)
		]
	];
}

void FPropertyBagInstanceDataDetails::AddChildRows(IDetailChildrenBuilder& ChildBuilder, const TArray<TSharedPtr<IPropertyHandle>>& ChildProperties)
{
	if (HierarchyData.HierarchyRoot.IsValid() && BagStructProperty.IsValid() && BagStructProperty->IsValidHandle())
	{		
		void* BagStructPtr = nullptr;
		FPropertyAccess::Result Result = BagStructProperty->GetValueData(BagStructPtr);
		
		if (Result == FPropertyAccess::Fail)
		{
			return;
		}
		
		FInstancedPropertyBag* Bag = static_cast<FInstancedPropertyBag*>(BagStructPtr);
		
		// We build a map from hierarchy property to property handle. This is then used in hierarchy category builders to retrieve their handles to add to the details panel.
		TArray<const UPropertyBagHierarchyProperty*> AllHierarchyProperties;
		HierarchyData.HierarchyRoot->GetChildrenOfType(AllHierarchyProperties, true);
		
		HierarchyData.HierarchyPropertyHandleMap.Empty();
		HierarchyData.PropertyPathPropertyMap.Empty();
		for(const UPropertyBagHierarchyProperty* HierarchyProperty : AllHierarchyProperties)
		{
			if (const FPropertyBagPropertyDesc* PropertyDesc = Bag->FindPropertyDescByID(HierarchyProperty->GetPropertyId()))
			{
				const TSharedPtr<IPropertyHandle>* MatchingPropertyHandle = ChildProperties.FindByPredicate([PropertyDesc](const TSharedPtr<IPropertyHandle>& Candidate)
				{
					return Candidate->GetProperty() == PropertyDesc->CachedProperty;
				});
				
				if (MatchingPropertyHandle)
				{
					HierarchyData.HierarchyPropertyHandleMap.Add(TObjectKey<const UPropertyBagHierarchyProperty>(HierarchyProperty), *MatchingPropertyHandle);
					HierarchyData.PropertyPathPropertyMap.Add(FString((*MatchingPropertyHandle)->GetPropertyPath()), HierarchyProperty);
				}
			}
		}
		
		// First, we generate the section row
		GenerateHierarchySectionRow(ChildBuilder);
		// Then the root level categories & properties
		GenerateHierarchyRootRows(ChildBuilder);
		// Then the leftover properties that have not been embedded into the hierarchy
		GenerateHierarchyLeftoverPropertyRows(ChildBuilder, ChildProperties);
		// Then the 'Show Advanced' row, if there is any advanced property
		GenerateHierarchyShowAdvancedRow(ChildBuilder);
	}
	else
	{
		FInstancedStructDataDetails::AddChildRows(ChildBuilder, ChildProperties);
	}
}

void FPropertyBagInstanceDataDetails::OnGroupRowAdded(IDetailGroup& GroupRow, int32 Level, const FString& Category) const
{
	using namespace UE::StructUtils;

	FDetailWidgetRow& FolderRow = GroupRow.HeaderRow();
	TWeakPtr<const FPropertyBagInstanceDataDetails> WeakSelf = SharedThis<const FPropertyBagInstanceDataDetails>(this);
	TWeakPtr<IPropertyHandle> WeakPropertyBagHandle = BagStructProperty;
	
	FString FullCategoryName = GroupRow.GetGroupName().ToString();

	/*** DRAG AND DROP HANDLER ***/
	if (EnumHasAllFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::DragAndDrop | EPropertyBagChildRowFeatures::Menu_Categories))
	{
		TSharedPtr<FPropertyBagDetailsDragDropHandlerTarget> DragDropHandler = MakeShared<FPropertyBagDetailsDragDropHandlerTarget>();

		DragDropHandler->BindCanAcceptDragDrop(
			FCanAcceptPropertyBagDetailsRowDropOp::CreateLambda(
				[FullCategoryName](const TSharedPtr<FPropertyBagDetailsDragDropOp>& DropOp, EItemDropZone DropZone) -> TOptional<EItemDropZone>
				{
					if (!DropOp.IsValid() || DropZone != EItemDropZone::OntoItem)
					{
						DropOp->SetDecoration(EPropertyBagDropState::Invalid);
						return TOptional<EItemDropZone>();
					}

					TOptional<FPropertyBagDetailsDragDropOp::FDecoration> DecorationOverride;

					if (Metadata::AreCategoriesEnabled(DropOp->PropertyDesc)
						&& Metadata::GetCategory(DropOp->PropertyDesc).Equals(FullCategoryName))
					{
						const FSlateBrush* Brush = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.OKWarn");
						DecorationOverride.Emplace(LOCTEXT("OnSameCategoryDragDropDecoratorMessage", "Already in this category"), Brush);
						DropOp->SetDecoration(EPropertyBagDropState::SourceIsTarget, std::move(DecorationOverride));
						return TOptional<EItemDropZone>();
					}

					const FSlateBrush* Brush = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.OK");
					DecorationOverride.Emplace(LOCTEXT("OnNewCategoryDragDropDecoratorMessage", "Move to this category"), Brush);
					DropOp->SetDecoration(EPropertyBagDropState::Valid, std::move(DecorationOverride));
					return DropZone;
				}));

		DragDropHandler->BindOnHandleDragDrop(
			FOnPropertyBagDetailsRowDropOp::CreateLambda(
				[WeakSelf, FullCategoryName, BagStructProperty = WeakPropertyBagHandle, PropUtils = PropUtils](const FPropertyBagPropertyDesc& DroppedPropertyDesc, const EItemDropZone DropZone) -> FReply
				{
					if (!BagStructProperty.IsValid())
					{
						return FReply::Unhandled();
					}
						
					if (ensure(DroppedPropertyDesc.CachedProperty && DropZone == EItemDropZone::OntoItem))
					{
						const TSharedPtr<const FPropertyBagInstanceDataDetails> DetailsSP = WeakSelf.Pin();
						const UPropertyBag* ChildBagStruct = DetailsSP ? GetCommonBagStruct(DetailsSP->BagStructProperty) : nullptr;
						// Validate these properties are still part of the bag.
						if (!ChildBagStruct || !ChildBagStruct->FindPropertyDescByProperty(DroppedPropertyDesc.CachedProperty))
						{
							return FReply::Unhandled();
						}

						ApplyChangesToSinglePropertyDesc(
							LOCTEXT("DragToChangeCategory", "Change property category"),
							DroppedPropertyDesc,
							BagStructProperty.Pin(),
							[&DroppedPropertyDesc, &FullCategoryName](FPropertyBagPropertyDesc& Desc)
							{
								Metadata::SetCategory(Desc, FullCategoryName);
							});

						return FReply::Handled();
					}

					return FReply::Unhandled();
				}));

		// Add the drag and drop handler as a target for the folder row.
		FolderRow.DragDropHandler(std::move(DragDropHandler));
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bIsFixed = bFixedLayout || (ChildRowFeatures == EPropertyBagChildRowFeatures::Fixed);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/*** EDITABLE NAME BLOCK ***/
	const TSharedPtr<SInlineEditableTextBlock> EditableInlineNameWidget = SNew(SInlineEditableTextBlock)
		.MultiLine(false)
		.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
		.Font(IDetailLayoutBuilder::GetDetailFontBold())
		.Text(FText::FromString(Category))
		.OnVerifyTextChanged_Lambda([](const FText& InText, FText& OutErrorMessage)
		{
			if (InText.IsEmpty())
			{
				OutErrorMessage = LOCTEXT("InlineEmptyCategoryName", "Name is empty");
				return false;
			}
			else if (InText.ToString().Len() > Constants::MaxCategoryLength)
			{
				OutErrorMessage = LOCTEXT("InlineInvalidCategoryLength", "Too many characters");
				return false;
			}
			else if (!FName::IsValidXName(InText.ToString(), Constants::InvalidCategoryCharacters))
			{
				OutErrorMessage = LOCTEXT("InlineInvalidCategoryName", "Invalid character(s)");
				return false;
			}

			return true;
		})
		.OnTextCommitted_Lambda([FullCategoryName, Category = Category, BagStructProperty = BagStructProperty, PropUtils = PropUtils](const FText& InNewText, const ETextCommit::Type InCommitType)
		{
			if (InCommitType == ETextCommit::OnEnter || InCommitType == ETextCommit::OnUserMovedFocus)
			{
				using namespace UE::StructUtils;

				ApplyChangesToPropertyDescs(
					LOCTEXT("InlineRenameCategory", "Rename category"),
					BagStructProperty,
					[&InNewText, OldCategory = FullCategoryName, &Category](TArray<FPropertyBagPropertyDesc>& PropertyDescs)
					{
						FString NewCategory = OldCategory;
						NewCategory.ReplaceInline(*Category, *InNewText.ToString(), ESearchCase::CaseSensitive);
						for (FPropertyBagPropertyDesc& Desc : PropertyDescs)
						{
							if (Metadata::AreCategoriesEnabled(Desc))
							{
								FString DescCategory = Metadata::GetCategory(Desc);
								if (DescCategory.StartsWith(OldCategory))
								{
									DescCategory.ReplaceInline(*OldCategory, *NewCategory);
									Metadata::SetCategory(Desc, DescCategory);
								}
							}
						}
					});
			}
		})
		.IsReadOnly(bIsFixed || !EnumHasAnyFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::Renaming));

	/*** CATEGORY NAME AND BUTTONS ***/
	const TSharedPtr<SBorder> NameContent = SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(1, 0)
			[
				EditableInlineNameWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SSpacer).Size(1)
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("DeleteCategory", "Delete this category."))
				.Visibility(bIsFixed ? EVisibility::Collapsed : EVisibility::Visible)
				.OnClicked_Lambda([GroupName = GroupRow.GetGroupName(), BagStructProperty = BagStructProperty, PropUtils = PropUtils]()
				{
					ApplyChangesToPropertyDescs(
						LOCTEXT("OnCategoryDeleted", "Delete category"),
						BagStructProperty,
						[GroupName](TArray<FPropertyBagPropertyDesc>& PropertyDescs)
						{
							for (FPropertyBagPropertyDesc& Desc : PropertyDescs)
							{
								if (Metadata::GetCategory(Desc).Equals(GroupName.ToString()))
								{
									Metadata::RemoveCategory(Desc);
								}
							}
						});

					return FReply::Handled();
				})
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				]
			]
		];

	// Mirrors PropertyEditorConstants::GetRowBackgroundColor, which is private.
	NameContent->SetBorderBackgroundColor(TAttribute<FSlateColor>::CreateLambda([NameContent, Level]()
	{
		int32 ColorIndex = 0;
		int32 Increment = 1;

		for (int i = 0; i < Level + 1; ++i)
		{
			ColorIndex += Increment;

			if (ColorIndex == 0 || ColorIndex == 3)
			{
				Increment = -Increment;
			}
		}

		static constexpr uint8 ColorOffsets[] =
		{
			0, 4, (4 + 2), (6 + 4), (10 + 6)
		};

		const FSlateColor BaseSlateColor = NameContent->IsHovered() ? FAppStyle::Get().GetSlateColor("Colors.Header") : FAppStyle::Get().GetSlateColor("Colors.Panel");

		const FColor BaseColor = BaseSlateColor.GetSpecifiedColor().ToFColor(true);

		const FColor ColorWithOffset(
			BaseColor.R + ColorOffsets[ColorIndex],
			BaseColor.G + ColorOffsets[ColorIndex],
			BaseColor.B + ColorOffsets[ColorIndex]);

		return FSlateColor(FLinearColor::FromSRGBColor(ColorWithOffset));
	}));

	FolderRow
		.ShouldAutoExpand(true)
		.WholeRowContent()
		.HAlign(HAlign_Fill)
		[
			NameContent.ToSharedRef()
		];
}

void FPropertyBagInstanceDataDetails::OnChildRowAdded(IDetailPropertyRow& ChildRow)
{
	using namespace UE::StructUtils;
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> PropertyValueWidget;
	FDetailWidgetRow DetailWidgetRow;

	TSharedPtr<IPropertyHandle> ChildPropertyHandle = ChildRow.GetPropertyHandle();
	check(ChildPropertyHandle);

	ChildRow.GetDefaultWidgets(NameWidget, PropertyValueWidget, DetailWidgetRow);
	// Wrap the widget in a box of the desired size, based on the property type.
	PropertyValueWidget = SNew(SBox)
		.MinDesiredWidth(Private::GetDesiredInputWidgetSizeByType(ChildPropertyHandle))
		.Content()
		[
			PropertyValueWidget.ToSharedRef()
		];

	TWeakPtr<FPropertyBagInstanceDataDetails> WeakSelf = SharedThis<FPropertyBagInstanceDataDetails>(this);

	const UPropertyBag* BagStruct = BagStructProperty ? GetCommonBagStruct(BagStructProperty) : nullptr;
	const FProperty* ChildProperty = ChildPropertyHandle->GetProperty();
	const FPropertyBagPropertyDesc* PropertyDesc = BagStruct ? BagStruct->FindPropertyDescByProperty(ChildProperty) : nullptr;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bIsFixed = bFixedLayout || ChildRowFeatures == EPropertyBagChildRowFeatures::Fixed;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Validate data and check if it's editable
	// If there's no common bag struct, that might just mean that Multiple Values are selected
	if (!ChildProperty || ChildProperty->HasMetaData(Metadata::HideInDetailPanelsName) || !BagStruct)
	{
		ChildRow.Visibility(EVisibility::Collapsed);
		return;
	}

	bool bEditable = BagStructProperty->IsEditable();

	/*** WARNINGS FOR PROPERTY ISSUES ***/
	FText WarningOnProperty; // This message will supplement a warning icon on the details view child row, which will show if not empty.
	if (!ensure(PropertyDesc) || PropertyDesc->ContainerTypes.Num() > 1)
	{
		// The property editing for nested containers is not supported.
		WarningOnProperty = LOCTEXT("NestedContainersWarning", "This property type (nested container) is not supported in the property bag UI.");
	}
	else if ((PropertyDesc->ValueType == EPropertyBagPropertyType::UInt32 || PropertyDesc->ValueType == EPropertyBagPropertyType::UInt64) && !bIsFixed)
	{
		// Warn that the unsigned types cannot be set via the type selection.
		WarningOnProperty = LOCTEXT("UnsignedTypesWarning", "Unsigned types are not supported through the property type selection. If you change the type, you will not be able to change it back.");
	}
	else if (Private::HasMissingType(ChildPropertyHandle))
	{
		WarningOnProperty = LOCTEXT("MissingTypeWarning", "The property is missing type. The Struct, Enum, or Object may have been removed.");
	}
	else if (!FInstancedPropertyBag::IsPropertyNameValid(Private::GetPropertyNameSafe(ChildPropertyHandle)))
	{
		WarningOnProperty = LOCTEXT("InvalidNameWarning", "The property's name contains invalid characters. Dynamically named properties with invalid characters may be rejected in future releases.");
	}

	/*** OVERRIDE RESET TO DEFAULT ACTION FOR BAG OVERRIDES ***/
	if (HasPropertyOverrides())
	{
		TAttribute<bool> EditConditionValue = TAttribute<bool>::CreateLambda(
			[WeakSelf, ChildPropertyHandle]() -> bool
			{
				if (const TSharedPtr<FPropertyBagInstanceDataDetails> Self = WeakSelf.Pin())
				{
					return Self->IsPropertyOverridden(ChildPropertyHandle) == EPropertyOverrideState::Yes;
				}
				return true;
			});

		FOnBooleanValueChanged OnEditConditionChanged = FOnBooleanValueChanged::CreateLambda([WeakSelf, ChildPropertyHandle](bool bNewValue)
		{
			if (const TSharedPtr<FPropertyBagInstanceDataDetails> Self = WeakSelf.Pin())
			{
				Self->SetPropertyOverride(ChildPropertyHandle, bNewValue);
			}
		});

		ChildRow.EditCondition(MoveTemp(EditConditionValue), MoveTemp(OnEditConditionChanged));

		FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateLambda([WeakSelf](TSharedPtr<IPropertyHandle> PropertyHandle)
		{
			if (const TSharedPtr<FPropertyBagInstanceDataDetails> Self = WeakSelf.Pin())
			{
				return !Self->IsDefaultValue(PropertyHandle);
			}
			return false;
		});
		FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateLambda([WeakSelf](TSharedPtr<IPropertyHandle> PropertyHandle)
		{
			if (const TSharedPtr<FPropertyBagInstanceDataDetails> Self = WeakSelf.Pin())
			{
				Self->ResetToDefault(PropertyHandle);
			}
		});
		FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);

		ChildRow.OverrideResetToDefault(ResetOverride);
	}

	if (!bIsFixed)
	{
		/*** BUILD PROPERTY NAME WIDGET ***/
		TSharedRef<SHorizontalBox> PropertyDetailsWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.Padding(3, 0)
			.AutoWidth()
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.ToolTipText(WarningOnProperty)
					.Visibility_Lambda([WarningOnProperty]()
					{
						return WarningOnProperty.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
					})
					.DesiredSizeOverride(FVector2D(12))
					.ColorAndOpacity(FLinearColor(1.f, 0.8f, 0.f, 1.f))
					.Image(FAppStyle::GetBrush("Icons.Error"))
				]
			];

		if (EnumHasAnyFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::CompactTypeSelector))
		{
			PropertyDetailsWidget->AddSlot()
				.HAlign(HAlign_Left)
				.Padding(1, 0)
				.AutoWidth()
				[
					CreateTypeSelectionWidget(ChildPropertyHandle, BagStructProperty, PropUtils, SPinTypeSelector::ESelectorType::Compact, bAllowContainers, GetPropertyBagSchemaClass())
				];
		}

		/*** EDITABLE NAME BLOCK ***/
		TSharedPtr<SInlineEditableTextBlock> EditableInlineNameWidget = SNew(SInlineEditableTextBlock)
			.IsReadOnly(!EnumHasAnyFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::Renaming))
			.ToolTipText_Lambda([ChildPropertyHandle]() -> FText
			{
				return ChildPropertyHandle->HasMetaData(Metadata::Specifiers::ToolTipName)
						   ? FText::FromString(ChildPropertyHandle->GetMetaData(Metadata::Specifiers::ToolTipName))
						   : FText::GetEmpty();
			})
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.MultiLine(false)
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.Text_Lambda([ChildPropertyHandle]()
			{
				const FName PropertyName = Private::GetPropertyNameSafe(ChildPropertyHandle);
				return FText::FromName(PropertyName);
			})
			.OnVerifyTextChanged_Lambda([BagStructProperty = BagStructProperty, ChildPropertyHandle](const FText& InText, FText& OutErrorMessage)
			{
				if (InText.IsEmpty())
				{
					OutErrorMessage = LOCTEXT("InlineEmptyPropertyName", "Name is empty");
					return false;
				}

				// Check for invalid characters upon renaming.
				if (!FInstancedPropertyBag::IsPropertyNameValid(InText.ToString()))
				{
					OutErrorMessage = LOCTEXT("InlineInvalidPropertyName", "Invalid character(s)");
					return false;
				}

				const FName OldName = Private::GetPropertyNameSafe(ChildPropertyHandle);
				// Bypass if the name is the exact same.
				if (InText.ToString().Equals(OldName.ToString()))
				{
					return true;
				}

				// Sanitize out any other characters that we allowed for convenience but are not valid, like spaces.
				const FName NewName = FInstancedPropertyBag::SanitizePropertyName(InText.ToString());

				// Bypass if sanitized name is the same.
				if (NewName == OldName)
				{
					return true;
				}

				if (!Private::IsUniqueName(NewName, OldName, BagStructProperty))
				{
					OutErrorMessage = LOCTEXT("InlinePropertyUniqueName", "Property must have unique name");
					return false;
				}

				// Name is OK.
				return true;
			})
			.OnTextCommitted_Lambda([BagStructProperty = BagStructProperty, PropUtils = PropUtils, ChildPropertyHandle](const FText& InNewText, ETextCommit::Type InCommitType)
			{
				if (InCommitType == ETextCommit::OnCleared)
				{
					return;
				}

				const FName NewName = FInstancedPropertyBag::SanitizePropertyName(InNewText.ToString());
				const FName OldName = Private::GetPropertyNameSafe(ChildPropertyHandle);

				if (NewName == OldName)
				{
					return;
				}

				if (!ensureMsgf(Private::IsUniqueName(NewName, OldName, BagStructProperty), TEXT("Should have already been addressed in OnVerifyTextChanged.")))
				{
					return;
				}

				ApplyChangesToPropertyDescs(
					FText::Format(LOCTEXT("OnPropertyNameChanged", "Change property name: {0} -> {1}"), FText::FromName(OldName), FText::FromName(NewName)),
					BagStructProperty,
					[&NewName, &ChildPropertyHandle](TArray<FPropertyBagPropertyDesc>& PropertyDescs)
					{
						const FProperty* Property = ChildPropertyHandle->GetProperty();
						if (FPropertyBagPropertyDesc* Desc = PropertyDescs.FindByPredicate([Property](const FPropertyBagPropertyDesc& Desc) { return Desc.CachedProperty == Property; }))
						{
							Desc->Name = NewName;
						}
					});
			});

		/*** CURRENT UI AS IT BECOMES DEPRECATED ***/
		// Deprecated in 5.6 - the combo button on the name widget will be removed in favor of the new drop-down menu.
		if (EnumHasAnyFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::Deprecated))
		{
			// Add the widget to the property bar.
			PropertyDetailsWidget->AddSlot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				.Padding(0, 0, 3, 0)
				[
					SNew(SBox)
					.Content()
					[
						SNew(SComboButton)
						.MenuContent()
						[
							PRAGMA_DISABLE_DEPRECATION_WARNINGS
							OnPropertyNameContent(ChildPropertyHandle, EditableInlineNameWidget)
							PRAGMA_ENABLE_DEPRECATION_WARNINGS
						]
						.ContentPadding(FMargin(0, 0, 2, 0))
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ButtonContent()
						[
							EditableInlineNameWidget.ToSharedRef()
						]
					]
				];
		}
		else // No deprecated combo box. Just add the name.
		{
			PropertyDetailsWidget->AddSlot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				.Padding(0, 0, 3, 0)
				[
					EditableInlineNameWidget.ToSharedRef()
				];
		}

		// Extendable spacer between the name and the drop-down
		PropertyDetailsWidget->AddSlot()
			.FillWidth(1)
			[
				SNew(SSpacer)
				.Size(1)
			];

		/*** ACCESS SPECIFIER BUTTON ***/
		if (EnumHasAnyFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::AccessSpecifierButton))
		{
			using namespace UE::StructUtils::Metadata;
			PropertyDetailsWidget->AddSlot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("SetAccessSpecifier", "Set the access specifier on the property to Public or Private."))
					.OnClicked_Lambda([ChildPropertyHandle, BagStructProperty = BagStructProperty, PropUtils = PropUtils]()
					{
						FProperty* Property = ChildPropertyHandle->GetProperty();
						ApplyChangesToSinglePropertyDesc(
							LOCTEXT("OnPropertyAccessSpecifierChanged", "Set access specifier."),
							ChildPropertyHandle,
							BagStructProperty,
							[Property, BagStructProperty](FPropertyBagPropertyDesc& Desc)
							{
								const bool bIsPrivate = !Property->HasAnyPropertyFlags(CPF_NativeAccessSpecifierPublic);
								Desc.PropertyFlags &= ~CPF_NativeAccessSpecifiers;
								if (bIsPrivate)
								{
									Desc.PropertyFlags |= CPF_NativeAccessSpecifierPublic;
								}
								else
								{
									Desc.PropertyFlags |= CPF_NativeAccessSpecifierPrivate;
								}
							});

						return FReply::Handled();
					})
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(16))
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image_Lambda([ChildPropertyHandle]()
						{
							check(ChildPropertyHandle);
							if (const FProperty* Property = ChildPropertyHandle->GetProperty())
							{
								// For now, treat protected as private. TODO: Add toggle for protected.
								if (Property->HasAnyPropertyFlags(CPF_NativeAccessSpecifierPublic))
								{
									return FAppStyle::Get().GetBrush("Icons.Visible");
								}
							}

							return FAppStyle::Get().GetBrush("Icons.Hidden");
						})
					]
				];
		}

		/*** DROP-DOWN MENU OPTIONS ***/
		// Check drop-down is enabled and at least one option as well.
		if (EnumHasAnyFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::DropDownMenuButton)
			&& EnumHasAnyFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::AllMenuOptions))
		{
			static constexpr bool bInShouldCloseWindowAfterMenuSelection = true;
			FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);

			if (EnumHasAnyFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::Menu_TypeSelector))
			{
				MenuBuilder.BeginSection(/*InExtensionHook=*/NAME_None, LOCTEXT("DropDownMenuSectionTypeSelector", "Type"));
				MenuBuilder.AddWidget(
					CreateTypeSelectionWidget(
						ChildPropertyHandle,
						BagStructProperty,
						PropUtils,
						SPinTypeSelector::ESelectorType::Full,
						bAllowContainers,
						GetPropertyBagSchemaClass()),
					/*InLabel=*/FText::GetEmpty());
				MenuBuilder.EndSection();
			}

			const bool bMenuRenameEnabled = EnumHasAllFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::Renaming | EPropertyBagChildRowFeatures::Menu_Rename);
			const bool bMenuDeleteEnabled = EnumHasAllFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::Deletion | EPropertyBagChildRowFeatures::Menu_Delete);

			if (bMenuRenameEnabled | bMenuDeleteEnabled)
			{
				MenuBuilder.BeginSection(/*InExtensionHook=*/NAME_None, LOCTEXT("DropDownMenuSectionGeneral", "General"));

				// Must have property renaming enabled or the editable inline widget will be invalid.
				if (bMenuRenameEnabled)
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("DropDownMenuRenameProperty", "Rename property"),
						LOCTEXT("DropDownMenuRenamePropertyToolTip", "Enable the inline property renaming."),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
						FUIAction(FExecuteAction::CreateLambda([NameWidgetWeak = EditableInlineNameWidget.ToWeakPtr()]()
						{
							if (TSharedPtr<SInlineEditableTextBlock> NameWidget = NameWidgetWeak.Pin())
							{
								NameWidget->EnterEditingMode();
							}
						})));
				}

				if (bMenuDeleteEnabled)
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("DropDownMenuRemoveProperty", "Remove property"),
						LOCTEXT("DropDownMenuRemovePropertyToolTip", "Delete the property."),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
						FUIAction(FExecuteAction::CreateLambda([BagStructProperty = BagStructProperty, PropUtils = PropUtils, ChildPropertyHandle]()
						{
							Private::DeleteProperty(BagStructProperty, ChildPropertyHandle, PropUtils);
						}))
					);
				}

				MenuBuilder.EndSection();
			}
			
			if (HierarchyData.HierarchyRoot.IsValid())
			{
				if (HierarchyData.OnNavigateToHierarchyPropertyRequestedDelegate.IsBound())
				{
					MenuBuilder.AddMenuEntry(
						LOCTEXT("DropDownMenuEditPropertyInHierarchyEditor", "Edit in Hierarchy Editor"),
						LOCTEXT("DropDownMenUEditPropertyInHierarchyEditorToolTip", "Edit this property in the Hierarchy Editor"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.EditHierarchy"),
						FUIAction(FExecuteAction::CreateLambda([this, PropertyDesc]()
						{
							HierarchyData.OnNavigateToHierarchyPropertyRequestedDelegate.Execute(*PropertyDesc);
						}))
					);
				}
			}

			// We only add metadata menu entries if we don't have a hierarchy root
			if (!HierarchyData.HierarchyRoot.IsValid())
			{
				// PropertyBagHandle and PropUtils are forwarded straight to OnTextCommitted, so they save a ref count and be refs here.
				auto BuildMetadataSpecifierSubMenu = [](
					TAttribute<FText>&& TextAttribute,
					FOnVerifyTextChanged&& OnVerifyTextDelegate,
					FOnTextCommitted&& OnTextCommittedDelegate) -> FNewMenuDelegate
				{
					TSharedRef<SWidget> TextBox = SNew(SMultiLineEditableTextBox)
						.WrapTextAt(400)
						.Text(MoveTemp(TextAttribute))
						.OnVerifyTextChanged(MoveTemp(OnVerifyTextDelegate))
						.OnTextCommitted(OnTextCommittedDelegate);

					return FNewMenuDelegate::CreateLambda(
						[TextBox](FMenuBuilder& OutMenuBuilder)
						{
							OutMenuBuilder.AddWidget(
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.HAlign(HAlign_Fill)
								.MinWidth(Constants::MinSubMenuTextBoxWidth)
								.MaxWidth(Constants::MaxSubMenuTextBoxWidth)
								.Padding(Constants::SubMenuTextBoxPadding, 0)
								.AutoWidth()
								[
									TextBox
								],
								FText::GetEmpty(),
								/*bNoIndent=*/true,
								/*bInSearchable=*/true);
						});
				};

				// The property's category (grouping) can be edited here.
				if (EnumHasAllFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::Categories | EPropertyBagChildRowFeatures::Menu_Categories))
				{
					MenuBuilder.BeginSection(/*InExtensionHook=*/NAME_None, LOCTEXT("DropDownMenuSectionCategory", "Category"));

					if (ChildPropertyHandle->HasMetaData(Metadata::CategoryName))
					{
						MenuBuilder.AddMenuEntry(
							LOCTEXT("DropDownMenuClearCategory", "Clear category"),
							LOCTEXT("DropDownMenuClearCategoryToolTip", "Remove the property from its current category."),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
							FUIAction(FExecuteAction::CreateLambda([ChildPropertyHandle, StructProperty = BagStructProperty, PropUtils = PropUtils]()
							{
								ApplyChangesToSinglePropertyDesc(
									LOCTEXT("DropDownMenuOnCategoryCleared", "Clear property category"),
									ChildPropertyHandle,
									StructProperty,
									[](FPropertyBagPropertyDesc& Desc)
										{
											Metadata::RemoveCategory(Desc);
										});
							})));
					}

					FOnVerifyTextChanged ValidateCategoryNameDelegate = FOnVerifyTextChanged::CreateLambda([](const FText& InText, FText& OutMessage)
					{
						if (InText.ToString().Len() > Constants::MaxCategoryLength)
						{
							OutMessage = LOCTEXT("DropDownMenuInvalidCategoryName", "Invalid category name");
							return false;
						}
						else
						{
							return true;
						}
					});

					MenuBuilder.AddSubMenu(
						FText::FromName(Metadata::CategoryName),
						LOCTEXT("DropDownMenuSubMenuCategoryTooltip", "Set the category of this property. Subcategories can be created with the '|' character."),
						BuildMetadataSpecifierSubMenu(
							TAttribute<FText>::CreateLambda([ChildPropertyHandle]()
							{
								FText GroupLabel = FText::FromString("");
								if (ChildPropertyHandle && ChildPropertyHandle->HasMetaData(Metadata::CategoryName))
								{
									GroupLabel = FText::FromString(ChildPropertyHandle->GetMetaData(Metadata::CategoryName));
								}

								return GroupLabel;
							}),
							FOnVerifyTextChanged::CreateLambda([](const FText& InText, FText& OutMessage)
							{
								if (InText.ToString().Len() > Constants::MaxCategoryLength)
								{
									OutMessage = LOCTEXT("DropDownMenuInvalidCategoryName", "Invalid category name");
									return false;
								}
								else
								{
									return true;
								}
							}),

							FOnTextCommitted::CreateLambda([StructProperty = BagStructProperty, PropUtils = PropUtils, ChildPropertyHandle](const FText& CommittedText, const ETextCommit::Type CommitType)
							{
								if (CommitType == ETextCommit::OnEnter || CommitType == ETextCommit::OnUserMovedFocus)
								{
									ApplyChangesToSinglePropertyDesc(
										LOCTEXT("DropDownMenuOnCategoryEdited", "Edit property category"),
										ChildPropertyHandle,
										StructProperty,
										[&CommittedText](FPropertyBagPropertyDesc& Desc)
											{
												Metadata::SetCategory(Desc, CommittedText.ToString());
											});
								}
							})),
						/*bInOpenSubMenuOnClick=*/true);

					MenuBuilder.EndSection();
				}

				// The property's optional metadata specifiers can be edited here.
				if (EnumHasAllFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::Menu_MetadataSpecifiers))
				{
					MenuBuilder.BeginSection(/*InExtensionHook=*/NAME_None, LOCTEXT("DropDownMenuSectionMetadataSpecifiers", "Specifiers"));

					auto BuildGenericSpecifierSubMenu = [this, ChildPropertyHandle, BuildMetadataSpecifierSubMenu](const FName& SpecifierName, const bool bIsNumeric = false)
					{
						return BuildMetadataSpecifierSubMenu(
							TAttribute<FText>::CreateLambda([SpecifierName, ChildPropertyHandle]()
							{
								return ChildPropertyHandle.IsValid()
										   ? FText::FromString(ChildPropertyHandle->GetMetaData(SpecifierName))
										   : FText::GetEmpty();
							}),
							!bIsNumeric
								? FOnVerifyTextChanged()
								: FOnVerifyTextChanged::CreateLambda([](const FText& InText, FText& OutMessage) -> bool
								{
									if (!InText.IsEmpty() && !InText.IsNumeric())
									{
										OutMessage = LOCTEXT("DropDownMenuSpecifierNumericValueExpected", "Value is not numeric");
										return false;
									}
									else
									{
										return true;
									}
								}),
							FOnTextCommitted::CreateLambda(
								[SpecifierName, StructProperty = BagStructProperty, PropUtils = PropUtils, ChildPropertyHandle]
								(const FText& CommittedText, const ETextCommit::Type CommitType)
								{
									if (CommitType != ETextCommit::OnCleared)
									{
										ApplyChangesToSinglePropertyDesc(
											FText::Format(LOCTEXT("DropDownMenuOnSetSpecifier", "Set metadata specifier '{0}' value"), FText::FromName(SpecifierName)),
											ChildPropertyHandle,
											StructProperty,
											[SpecifierName, &CommittedText](FPropertyBagPropertyDesc& Desc)
												{
													// If empty, remove the specifier
													if (CommittedText.IsEmpty())
													{
														Desc.RemoveMetadata(SpecifierName);
													}
													else
													{
														Desc.SetMetaData(SpecifierName, CommittedText.ToString());
													}
												});
									}
								}));
					};

					MenuBuilder.AddSubMenu(
						FText::FromName(Metadata::Specifiers::ToolTipName),
						LOCTEXT("DropDownMenuSubMenuTooltipTooltip", "Set the tooltip for this property."),
						BuildGenericSpecifierSubMenu(Metadata::Specifiers::ToolTipName),
						/*bInOpenSubMenuOnClick=*/true);

					if (Metadata::Specifiers::SupportsClamping(*PropertyDesc))
					{
						MenuBuilder.AddSubMenu(
							FText::FromName(Metadata::Specifiers::ClampMinName),
							LOCTEXT("DropDownMenuSubMenuClampMinTooltip", "Set the min value for this property."),
							BuildGenericSpecifierSubMenu(Metadata::Specifiers::ClampMinName, /*bIsNumeric=*/true),
							/*bInOpenSubMenuOnClick=*/true);

						MenuBuilder.AddSubMenu(
							FText::FromName(Metadata::Specifiers::ClampMaxName),
							LOCTEXT("DropDownMenuSubMenuClampMaxTooltip", "Set the max value for this property."),
							BuildGenericSpecifierSubMenu(Metadata::Specifiers::ClampMaxName, /*bIsNumeric=*/true),
							/*bInOpenSubMenuOnClick=*/true);

						MenuBuilder.AddSubMenu(
							FText::FromName(Metadata::Specifiers::UIMinName),
							LOCTEXT("DropDownMenuSubMenuUIMinTooltip", "Set the min UI value for this property."),
							BuildGenericSpecifierSubMenu(Metadata::Specifiers::UIMinName, /*bIsNumeric=*/true),
							/*bInOpenSubMenuOnClick=*/true);

						MenuBuilder.AddSubMenu(
							FText::FromName(Metadata::Specifiers::UIMaxName),
							LOCTEXT("DropDownMenuSubMenuUIMaxTooltip", "Set the max UI value for this property."),
							BuildGenericSpecifierSubMenu(Metadata::Specifiers::UIMaxName, /*bIsNumeric=*/true),
							/*bInOpenSubMenuOnClick=*/true);
					}

					if (Metadata::Specifiers::SupportsSettingUnits(*PropertyDesc))
					{
						MenuBuilder.AddSubMenu(
							FText::FromName(Metadata::Specifiers::UnitsName),
							LOCTEXT("DropDownMenuSubMenuUnitsTooltip", "Set the units descriptor for this property."),
							BuildGenericSpecifierSubMenu(Metadata::Specifiers::UnitsName),
							/*bInOpenSubMenuOnClick=*/true);
					}

					MenuBuilder.EndSection();
				}
			}

			/*** DROP-DOWN ARROW MENU ***/
			PropertyDetailsWidget->AddSlot()
				.HAlign(HAlign_Right)
				.AutoWidth()
				.Padding(0, 0, 5, 0)
				[
					SNew(SBox)
					.Content()
					[
						SNew(SComboButton)
						.MenuContent()
						[
							MenuBuilder.MakeWidget()
						]
						.HasDownArrow(true)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ButtonContent()
						[
							SNullWidget::NullWidget
						]
					]
				];
		}

		/** DRAG AND DROP HANDLER */
		if (EnumHasAnyFlags(ChildRowFeatures, EPropertyBagChildRowFeatures::DragAndDrop) && PropertyDesc != nullptr)
		{
			TSharedPtr<FPropertyBagDetailsDragDropHandler> DragDropHandler = MakeShared<FPropertyBagDetailsDragDropHandler>(*PropertyDesc);

			if (HierarchyData.HierarchyRoot.IsValid() && HierarchyData.HierarchyViewModel.IsValid())
			{
				// Hierarchy-based drag & drop: delegate to the hierarchy view model API
				UPropertyBagHierarchyViewModel* HierarchyVM = HierarchyData.HierarchyViewModel.Get();
				TSharedPtr<FHierarchyRootViewModel> RootVM = HierarchyVM->GetHierarchyRootViewModel();

				// Look up the FHierarchyElementViewModel for this property
				FHierarchyElementIdentity Identity = UPropertyBagHierarchyProperty::ConstructIdentity(*PropertyDesc);
				TSharedPtr<FHierarchyElementViewModel> ElementVM = RootVM ? RootVM->FindViewModelForChild(Identity, true) : nullptr;

				if (ElementVM.IsValid())
				{
					// Set the hierarchy element on the handler so it's attached to drag ops
					DragDropHandler->SetHierarchyElement(ElementVM);

					// Can accept drag and drop — delegate to CanDropOn
					DragDropHandler->BindCanAcceptDragDrop(
						FCanAcceptPropertyBagDetailsRowDropOp::CreateLambda(
							[ElementVM](const TSharedPtr<FPropertyBagDetailsDragDropOp>& DropOp, EItemDropZone DropZone) -> TOptional<EItemDropZone>
							{
								if (!DropOp.IsValid())
								{
									return TOptional<EItemDropZone>();
								}

								TSharedPtr<FHierarchyElementViewModel> DraggedElementVM = DropOp->GetHierarchyElement();
								if (!DraggedElementVM.IsValid())
								{
									DropOp->SetDecoration(EPropertyBagDropState::Invalid);
									return TOptional<EItemDropZone>();
								}

								FHierarchyElementViewModel::FResultWithUserFeedback Result = ElementVM->CanDropOn(DraggedElementVM, DropZone);
								if (Result.bResult)
								{
									TOptional<FPropertyBagDetailsDragDropOp::FDecoration> DecorationOverride;
									if (Result.UserFeedback.IsSet())
									{
										const FSlateBrush* Brush = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.OK");
										DecorationOverride.Emplace(Result.UserFeedback.GetValue(), Brush);
									}
									DropOp->SetDecoration(EPropertyBagDropState::Valid, MoveTemp(DecorationOverride));
									return DropZone;
								}
								else
								{
									TOptional<FPropertyBagDetailsDragDropOp::FDecoration> DecorationOverride;
									if (Result.UserFeedback.IsSet())
									{
										const FSlateBrush* Brush = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.Error");
										DecorationOverride.Emplace(Result.UserFeedback.GetValue(), Brush);
									}
									DropOp->SetDecoration(EPropertyBagDropState::Invalid, MoveTemp(DecorationOverride));
									return TOptional<EItemDropZone>();
								}
							}));

					// Handle the drop — delegate to OnDroppedOn. Uses the hierarchy element directly from the op.
					DragDropHandler->BindOnHandleHierarchyElementDrop(
						FOnPropertyBagHierarchyElementDropOp::CreateLambda(
							[ElementVM](TSharedPtr<FHierarchyElementViewModel> DroppedElementVM, EItemDropZone DropZone) -> FReply
							{
								if (!DroppedElementVM.IsValid())
								{
									return FReply::Unhandled();
								}

								ElementVM->OnDroppedOn(DroppedElementVM, DropZone);
								return FReply::Handled();
							}));
				}
			}
			else if (!HierarchyData.HierarchyRoot.IsValid())
			{
				// Non-hierarchy path: reorder via FInstancedPropertyBag::ReorderProperty
				DragDropHandler->BindCanAcceptDragDrop(
					FCanAcceptPropertyBagDetailsRowDropOp::CreateLambda(
						[PropertyDesc](const TSharedPtr<FPropertyBagDetailsDragDropOp>& DropOp, EItemDropZone DropZone) -> TOptional<EItemDropZone>
						{
							if (!PropertyDesc || !DropOp.IsValid())
							{
								DropOp->SetDecoration(EPropertyBagDropState::Invalid);
								return TOptional<EItemDropZone>();
							}

							if (DropZone == EItemDropZone::OntoItem && PropertyDesc->ID != DropOp->PropertyDesc.ID)
							{
								DropOp->SetDecoration(EPropertyBagDropState::Invalid);
								return TOptional<EItemDropZone>();
							}

							// No effect to drop in these cases. Either source == target, or moving source above/below target puts source in same location.
							if (*PropertyDesc == DropOp->PropertyDesc
								|| (DropZone == EItemDropZone::AboveItem && DropOp->PropertyDesc.GetCachedIndex() == PropertyDesc->GetCachedIndex() - 1)
								|| (DropZone == EItemDropZone::BelowItem && DropOp->PropertyDesc.GetCachedIndex() == PropertyDesc->GetCachedIndex() + 1))
							{
								DropOp->SetDecoration(EPropertyBagDropState::SourceIsTarget);
								return TOptional<EItemDropZone>();
							}

							DropOp->SetDecoration(EPropertyBagDropState::Valid);
							return DropZone;
						}));

				DragDropHandler->BindOnHandleDragDrop(
					FOnPropertyBagDetailsRowDropOp::CreateLambda(
						[WeakSelf, PropertyDesc = *PropertyDesc, PropertyBagHandle = BagStructProperty, PropUtils = PropUtils](const FPropertyBagPropertyDesc& DroppedPropertyDesc, EItemDropZone DropZone) -> FReply
						{
							using namespace UE::StructUtils;

							const TSharedPtr<FPropertyBagInstanceDataDetails> DetailsSP = WeakSelf.Pin();
							const UPropertyBag* ChildBagStruct = DetailsSP ? GetCommonBagStruct(DetailsSP->BagStructProperty) : nullptr;
							// Validate these properties are still part of the bag.
							if (!ChildBagStruct
								|| !ChildBagStruct->FindPropertyDescByProperty(PropertyDesc.CachedProperty)
								|| !ChildBagStruct->FindPropertyDescByProperty(DroppedPropertyDesc.CachedProperty))
							{
								return FReply::Unhandled();
							}

							EPropertyBagAlterationResult Result = EPropertyBagAlterationResult::InternalError;

							DetailsSP->BagStructProperty->EnumerateRawData([DropZone, &PropertyDesc, &DroppedPropertyDesc, &Result](void* RawData, const int32 /*DataIndex*/, const int32 /*NumData*/)
							{
								if (FInstancedPropertyBag* PropertyBag = RawData ? static_cast<FInstancedPropertyBag*>(RawData) : nullptr)
								{
									Result = PropertyBag->ReorderProperty(DroppedPropertyDesc.Name, PropertyDesc.Name, DropZone == EItemDropZone::AboveItem);
								}

								return true;
							});

							if (Result == EPropertyBagAlterationResult::Success)
							{
								ApplyChangesToSinglePropertyDesc(
									LOCTEXT("DragDropReorderProperties", "Reordered properties"),
									DroppedPropertyDesc,
									PropertyBagHandle,
									[PropertyDesc](FPropertyBagPropertyDesc& Desc)
										{
											Metadata::SetCategory(Desc, Metadata::GetCategory(PropertyDesc));
										});

								return FReply::Handled();
							}
							else
							{
								return FReply::Unhandled();
							}
						}));
			}

			// Bind the drag and drop handler for receiving.
			ChildRow.DragDropHandler(DragDropHandler);

			// Add draggability for the name widget.
			NameWidget = SNew(StructUtilsEditor::SDraggableBox)
				.DragDropHandler(DragDropHandler)
				.RequireDirectHover(true)
				.Content()
				[
					PropertyDetailsWidget
				];

			// Add draggability for the value widget, maximizing draggable space, but not at the cost of the value widget.
			PropertyValueWidget = SNew(StructUtilsEditor::SDraggableBox)
				.DragDropHandler(DragDropHandler)
				.RequireDirectHover(true)
				.Content()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.AutoWidth()
					[
						PropertyValueWidget.ToSharedRef()
					]
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.FillWidth(1)
					[
						SNullWidget::NullWidget
					]
				];
		}
		else
		{
			// Update the name widget with our new property details composition.
			NameWidget = PropertyDetailsWidget;
		}
	}

	/*** FINAL WIDGET ***/
	ChildRow
		.IsEnabled(bEditable)
		.CustomWidget(/*bShowChildren=*/true)
		.NameContent()
		.HAlign(HAlign_Fill)
		[
			NameWidget.ToSharedRef()
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			PropertyValueWidget.ToSharedRef()
		];
}

FPropertyBagInstanceDataDetails::EPropertyOverrideState FPropertyBagInstanceDataDetails::IsPropertyOverridden(TSharedPtr<IPropertyHandle> ChildPropertyHandle) const
{
	if (!ChildPropertyHandle)
	{
		return EPropertyOverrideState::Undetermined;
	}

	const FProperty* Property = ChildPropertyHandle->GetProperty();
	if (!Property)
	{
		return EPropertyOverrideState::Undetermined;
	}

	int32 NumValues = 0;
	int32 NumOverrides = 0;

	EnumeratePropertyBags(BagStructProperty,
		[Property, &NumValues, &NumOverrides]
		(const FInstancedPropertyBag& DefaultPropertyBag, const FInstancedPropertyBag& PropertyBag, const IPropertyBagOverrideProvider& OverrideProvider)
		{
			NumValues++;
			if (const UPropertyBag* Bag = PropertyBag.GetPropertyBagStruct())
			{
				const FPropertyBagPropertyDesc* PropertyDesc = Bag->FindPropertyDescByProperty(Property);
				if (PropertyDesc && OverrideProvider.IsPropertyOverridden(PropertyDesc->ID))
				{
					NumOverrides++;
				}
			}

			return true;
		});

	if (NumOverrides == 0)
	{
		return EPropertyOverrideState::No;
	}
	else if (NumOverrides == NumValues)
	{
		return EPropertyOverrideState::Yes;
	}
	return EPropertyOverrideState::Undetermined;
}

void FPropertyBagInstanceDataDetails::SetPropertyOverride(TSharedPtr<IPropertyHandle> ChildPropertyHandle, const bool bIsOverridden)
{
	if (!ChildPropertyHandle)
	{
		return;
	}

	const FProperty* Property = ChildPropertyHandle->GetProperty();
	check(Property);

	FScopedTransaction Transaction(FText::Format(LOCTEXT("OverrideChange", "Change Override for {0}"), FText::FromName(ChildPropertyHandle->GetProperty()->GetFName())));

	PreChangeOverrides();

	EnumeratePropertyBags(
		BagStructProperty,
		[Property, bIsOverridden]
		(const FInstancedPropertyBag& DefaultPropertyBag, const FInstancedPropertyBag& PropertyBag, const IPropertyBagOverrideProvider& OverrideProvider)
		{
			if (const UPropertyBag* Bag = PropertyBag.GetPropertyBagStruct())
			{
				if (const FPropertyBagPropertyDesc* PropertyDesc = Bag->FindPropertyDescByProperty(Property))
				{
					OverrideProvider.SetPropertyOverride(PropertyDesc->ID, bIsOverridden);
				}
			}

			return true;
		});

	PostChangeOverrides();
}

bool FPropertyBagInstanceDataDetails::IsDefaultValue(TSharedPtr<IPropertyHandle> ChildPropertyHandle) const
{
	if (!ChildPropertyHandle)
	{
		return true;
	}

	int32 NumValues = 0;
	int32 NumOverridden = 0;
	int32 NumIdentical = 0;

	const FProperty* Property = ChildPropertyHandle->GetProperty();
	check(Property);

	EnumeratePropertyBags(
		BagStructProperty,
		[Property, &NumValues, &NumOverridden, &NumIdentical]
		(const FInstancedPropertyBag& DefaultPropertyBag, const FInstancedPropertyBag& PropertyBag, const IPropertyBagOverrideProvider& OverrideProvider)
		{
			NumValues++;

			const UPropertyBag* DefaultBag = DefaultPropertyBag.GetPropertyBagStruct();
			const UPropertyBag* Bag = PropertyBag.GetPropertyBagStruct();
			if (Bag && DefaultBag)
			{
				const FPropertyBagPropertyDesc* PropertyDesc = Bag->FindPropertyDescByProperty(Property);
				const FPropertyBagPropertyDesc* DefaultPropertyDesc = DefaultBag->FindPropertyDescByProperty(Property);
				if (PropertyDesc
					&& DefaultPropertyDesc
					&& OverrideProvider.IsPropertyOverridden(PropertyDesc->ID))
				{
					NumOverridden++;
					if (UE::StructUtils::Private::ArePropertiesIdentical(DefaultPropertyDesc, DefaultPropertyBag, PropertyDesc, PropertyBag))
					{
						NumIdentical++;
					}
				}
			}
			return true;
		});

	if (NumOverridden == NumIdentical)
	{
		return true;
	}

	return false;
}

void FPropertyBagInstanceDataDetails::ResetToDefault(TSharedPtr<IPropertyHandle> ChildPropertyHandle)
{
	if (!ChildPropertyHandle)
	{
		return;
	}

	const FProperty* Property = ChildPropertyHandle->GetProperty();
	check(Property);

	FScopedTransaction Transaction(FText::Format(LOCTEXT("ResetToDefault", "Reset {0} to default value"), FText::FromName(ChildPropertyHandle->GetProperty()->GetFName())));
	ChildPropertyHandle->NotifyPreChange();

	EnumeratePropertyBags(
		BagStructProperty,
		[Property]
		(const FInstancedPropertyBag& DefaultPropertyBag, FInstancedPropertyBag& PropertyBag, const IPropertyBagOverrideProvider& OverrideProvider)
		{
			const UPropertyBag* DefaultBag = DefaultPropertyBag.GetPropertyBagStruct();
			const UPropertyBag* Bag = PropertyBag.GetPropertyBagStruct();
			if (Bag && DefaultBag)
			{
				const FPropertyBagPropertyDesc* PropertyDesc = Bag->FindPropertyDescByProperty(Property);
				const FPropertyBagPropertyDesc* DefaultPropertyDesc = DefaultBag->FindPropertyDescByProperty(Property);
				if (PropertyDesc
					&& DefaultPropertyDesc
					&& OverrideProvider.IsPropertyOverridden(PropertyDesc->ID))
				{
					UE::StructUtils::Private::CopyPropertyValue(DefaultPropertyDesc, DefaultPropertyBag, PropertyDesc, PropertyBag);
				}
			}
			return true;
		});

	ChildPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	ChildPropertyHandle->NotifyFinishedChangingProperties();
}

EVisibility FPropertyBagInstanceDataDetails::ShouldShowHierarchyProperty(TSharedPtr<IPropertyHandle> ChildPropertyHandle) const
{
	if (bShouldShowAdvanced)
	{
		return EVisibility::Visible;
	}
	
	// We read directly from the common metadata to check if a property is advanced since the property advanced flag might go out of sync.
	// This can happen when a property is deleted somehow (parent category deleted), which will re-add at the root automatically but with new metadata
	// The advanced property flag is pointless anyway in customized UI.
	if (const TWeakObjectPtr<const UPropertyBagHierarchyProperty>* Property = HierarchyData.PropertyPathPropertyMap.Find(FString(ChildPropertyHandle->GetPropertyPath())))
	{
		if (Property != nullptr && Property->IsValid())
		{
			if (const FPropertyBagPropertyMetadata_Common* CommonMetaData = (*Property)->FindPropertyMetaDataOfType<FPropertyBagPropertyMetadata_Common>())
			{
				return CommonMetaData->bAdvanced ? EVisibility::Collapsed : EVisibility::Visible;
			}
		}
	}
	
	return EVisibility::Visible;
}

TWeakObjectPtr<const UHierarchySection> FPropertyBagInstanceDataDetails::GetActiveHierarchySection() const
{
	return HierarchyData.ActiveHierarchySection;
}

const TMap<TObjectKey<const UPropertyBagHierarchyProperty>, TSharedPtr<IPropertyHandle>>& FPropertyBagInstanceDataDetails::GetHierarchyPropertyHandleMap() const
{
	return HierarchyData.HierarchyPropertyHandleMap;
}

UPropertyBagHierarchyViewModel* FPropertyBagInstanceDataDetails::GetHierarchyViewModel() const
{
	return HierarchyData.HierarchyViewModel.Get();
}

TSharedRef<SWidget> FPropertyBagInstanceDataDetails::OnPropertyNameContent(TSharedPtr<IPropertyHandle> ChildPropertyHandle, TSharedPtr<SInlineEditableTextBlock> InlineWidget) const
{
	constexpr bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);

	auto MoveProperty = [PropertyBagHandle = BagStructProperty, PropUtils = PropUtils, ChildPropertyHandle](const int32 Delta)
	{
		if (!PropertyBagHandle || !PropertyBagHandle->IsValidHandle() || !ChildPropertyHandle || !ChildPropertyHandle->IsValidHandle())
		{
			return;
		}

		UE::StructUtils::ApplyChangesToPropertyDescs(
			LOCTEXT("OnPropertyMoved", "Move Property"), PropertyBagHandle, [&ChildPropertyHandle, &Delta](TArray<FPropertyBagPropertyDesc>& PropertyDescs)
				{
					// Move
					if (PropertyDescs.Num() > 1)
					{
						const FProperty* Property = ChildPropertyHandle ? ChildPropertyHandle->GetProperty() : nullptr;
						const int32 PropertyIndex = PropertyDescs.IndexOfByPredicate([Property](const FPropertyBagPropertyDesc& Desc){ return Desc.CachedProperty == Property; });
						if (PropertyIndex != INDEX_NONE)
						{
							const int32 NewPropertyIndex = FMath::Clamp(PropertyIndex + Delta, 0, PropertyDescs.Num() - 1);
							PropertyDescs.Swap(PropertyIndex, NewPropertyIndex);
						}
					}
				});
	};

	MenuBuilder.AddWidget(
		SNew(SBox)
		.HAlign(HAlign_Right)
		.Padding(FMargin(12, 0, 12, 0))
		[
			UE::StructUtils::CreateTypeSelectionWidget(ChildPropertyHandle,
				BagStructProperty,
				PropUtils,
				SPinTypeSelector::ESelectorType::Full,
				bAllowContainers,
				GetPropertyBagSchemaClass())
		],
		FText::GetEmpty());

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("Rename", "Rename"),
		LOCTEXT("Rename_ToolTip", "Rename property"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
		FUIAction(FExecuteAction::CreateLambda([InlineWidget]()  { InlineWidget->EnterEditingMode(); }))
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("Remove", "Remove"),
		LOCTEXT("Remove_ToolTip", "Remove property"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
		FUIAction(FExecuteAction::CreateLambda([BagStructProperty = BagStructProperty, PropUtils = PropUtils, ChildPropertyHandle]()
		{
			UE::StructUtils::Private::DeleteProperty(BagStructProperty, ChildPropertyHandle, PropUtils);
		}))
	);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MoveUp", "Move Up"),
		LOCTEXT("MoveUp_ToolTip", "Move property up in the list"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ArrowUp"),
		FUIAction(FExecuteAction::CreateLambda(MoveProperty, -1))
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MoveDown", "Move Down"),
		LOCTEXT("MoveDown_ToolTip", "Move property down in the list"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ArrowDown"),
		FUIAction(FExecuteAction::CreateLambda(MoveProperty, +1))
	);

	return MenuBuilder.MakeWidget();
}

TSubclassOf<UPropertyBagSchema> FPropertyBagInstanceDataDetails::GetPropertyBagSchemaClass() const
{
	return PropertyBagSchema.IsValid() ? PropertyBagSchema->GetClass() : UPropertyBagSchema::StaticClass();
}

void FPropertyBagInstanceDataDetails::GenerateHierarchySectionRow(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (!HierarchyData.HierarchyRoot.IsValid())
	{
		return;
	}
	
	TArray<const UHierarchySection*> OrderedSections;
	TMap<const UHierarchySection*, TArray<const UHierarchyCategory*>> SectionCategoryMap;

	UHierarchyRoot* Root = HierarchyData.HierarchyRoot.Get();
	for(const UHierarchyElement* Child : Root->GetChildren())
	{
		if(const UHierarchyCategory* HierarchyCategory = Cast<UHierarchyCategory>(Child))
		{
			const FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = HierarchyCategory->FindMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
			if(SectionAssociation && SectionAssociation->Section.IsValid())
			{
				SectionCategoryMap.FindOrAdd(SectionAssociation->Section.Get()).Add(HierarchyCategory);	
			}
		}
	}

	// we create a custom row for user param sections here, but only if we have at least one section specified
	if(SectionCategoryMap.Num() > 0)
	{
		TSharedRef<SWrapBox> SectionsBox = SNew(SWrapBox).UseAllottedSize(true);

		// maps don't guarantee order, so we iterate over the original section data instead
		for(const TObjectPtr<UHierarchySection>& Section : Root->GetSectionData())
		{
			// if the map doesn't contain at the entry, it means there are 0 categories for that section, so we skip it
			if(!SectionCategoryMap.Contains(Section))
			{
				continue;
			}
			
			TArray<const UHierarchyCategory*>& HierarchyCategories = SectionCategoryMap[Section];
			
			if(HierarchyCategories.Num() > 0)
			{
				// we only want to add a section if any of its categories contain at least one parameter
				bool bDoesUserParamExist = false;
				for(const UHierarchyCategory* Category : HierarchyCategories)
				{
					bDoesUserParamExist |= Category->DoesOneChildExist<UPropertyBagHierarchyProperty>(true);

					if(bDoesUserParamExist)
					{
						break;
					}
				}

				if(bDoesUserParamExist)
				{
					SectionsBox->AddSlot()
					.Padding(FMargin(2.f))
					[
						SNew(SBox)
						.Padding(FMargin(0.f, 4.f, 0.f, 0.f))
						[
							SNew(SCheckBox)
							.Style(FAppStyle::Get(), "DetailsView.SectionButton")
							.OnCheckStateChanged_Lambda([this, Section](ECheckBoxState NewState)
							{
								HierarchyData.ActiveHierarchySection = Section;
								OnRegenerateChildren.ExecuteIfBound();
							})
							.IsChecked_Lambda([this, Section]()
							{
								return HierarchyData.ActiveHierarchySection == Section ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
							})
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "SmallText")
								.Text_UObject(Section.Get(), &UHierarchySection::GetSectionNameAsText)
								.ToolTipText_UObject(Section.Get(), &UHierarchySection::GetTooltip)
							]
						]
					];
				}
			}
		}

		// if we have at least one custom section, we add a default "All" section
		if(SectionsBox->GetChildren()->Num() > 0)
		{
			SectionsBox->AddSlot()
			.Padding(FMargin(2.f))
			[
				SNew(SBox)
				.Padding(FMargin(0.f, 4.f, 0.f, 0.f))
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "DetailsView.SectionButton")
					.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
					{
						HierarchyData.ActiveHierarchySection = nullptr;
						OnRegenerateChildren.ExecuteIfBound();
					})
					.IsChecked_Lambda([this]()
					{
						return HierarchyData.ActiveHierarchySection == nullptr ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "SmallText")
						.Text(FText::FromString("All"))
					]
				]
			];
			
			FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(FText::FromString("Sections"));
			Row.RowTag(FName("Sections"));
			
			Row.WholeRowContent()
			[
				SectionsBox
			];
		}
	}
}

void FPropertyBagInstanceDataDetails::GenerateHierarchyRootRows(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (!HierarchyData.HierarchyRoot.IsValid())
	{
		return;
	}
	
	// Then, we generate root level categories...
	TArray<const UPropertyBagHierarchyCategory*> Categories;
	HierarchyData.HierarchyRoot->GetChildrenOfType(Categories, false);
		
	for(const UPropertyBagHierarchyCategory* Category : Categories)
	{
		const UHierarchySection* Section = nullptr;
		if(const FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = Category->FindMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>())
		{
			if(SectionAssociation->Section.IsValid())
			{
				Section = SectionAssociation->Section.Get();
			}
		}
			
		// ... but skip categories where the section doesn't match or categories that don't actually contain a property
		if((HierarchyData.ActiveHierarchySection == nullptr || Section == HierarchyData.ActiveHierarchySection) && Category->DoesOneChildExist<UPropertyBagHierarchyProperty>(true))
		{
			ChildrenBuilder.AddCustomBuilder(MakeShared<FHierarchyPropertyCategoryBuilder>(*Category, SharedThis(this)));
		}
	}
		
	// Then we add root level properties
	TArray<const UPropertyBagHierarchyProperty*> HierarchyPropertiesUnderRoot;
	HierarchyData.HierarchyRoot->GetChildrenOfType(HierarchyPropertiesUnderRoot, false);
	for(const UPropertyBagHierarchyProperty* HierarchyProperty : HierarchyPropertiesUnderRoot)
	{
		// we only want to display parameters within the root in the all section
		if(HierarchyData.ActiveHierarchySection == nullptr)
		{
			if (HierarchyData.HierarchyPropertyHandleMap.Contains(HierarchyProperty))
			{
				IDetailPropertyRow& PropertyRow = ChildrenBuilder.AddProperty(HierarchyData.HierarchyPropertyHandleMap[HierarchyProperty].ToSharedRef());
				PropertyRow.Visibility(TAttribute<EVisibility>::CreateSP(this, &FPropertyBagInstanceDataDetails::ShouldShowHierarchyProperty, PropertyRow.GetPropertyHandle()));
				OnChildRowAdded(PropertyRow);
			}
		}
	}
}

void FPropertyBagInstanceDataDetails::GenerateHierarchyLeftoverPropertyRows(IDetailChildrenBuilder& ChildrenBuilder, const TArray<TSharedPtr<IPropertyHandle>>& AllChildPropertyHandles)
{
	if (!HierarchyData.HierarchyRoot.IsValid())
	{
		return;
	}
	
	// Then we add all parameters that haven't been setup in the hierarchy at all, if the active section is set to "All"
	if (HierarchyData.ActiveHierarchySection == nullptr)
	{
		TArray<const UPropertyBagHierarchyProperty*> AllHierarchyProperties;
		HierarchyData.HierarchyRoot->GetChildrenOfType(AllHierarchyProperties, true);
		
		TArray<TSharedPtr<IPropertyHandle>> LeftoverProperties = AllChildPropertyHandles;
		TArray<TSharedPtr<IPropertyHandle>> PropertyHandlesInHierarchy;
		HierarchyData.HierarchyPropertyHandleMap.GenerateValueArray(PropertyHandlesInHierarchy);
			
		LeftoverProperties.RemoveAll([PropertyHandlesInHierarchy](TSharedPtr<IPropertyHandle> RemovalCandidate)
		{
			return PropertyHandlesInHierarchy.Contains(RemovalCandidate);
		});
			
		for (TSharedPtr<IPropertyHandle> LeftoverProperty : LeftoverProperties)
		{
			IDetailPropertyRow& PropertyRow = ChildrenBuilder.AddProperty(LeftoverProperty.ToSharedRef());
			PropertyRow.Visibility(TAttribute<EVisibility>::CreateSP(this, &FPropertyBagInstanceDataDetails::ShouldShowHierarchyProperty, PropertyRow.GetPropertyHandle()));
			OnChildRowAdded(PropertyRow);
		}
	}
}

const FSlateBrush* FPropertyBagInstanceDataDetails::GetAdvancedImage() const
{	
	FName ResourceName;
	if (bShouldShowAdvanced)
	{
		if (ShowAdvancedButton->IsHovered())
		{
			static const FName ExpandedHoveredName = "DetailsView.PulldownArrow.Up.Hovered";
			ResourceName = ExpandedHoveredName;
		}
		else
		{
			static const FName ExpandedName = "DetailsView.PulldownArrow.Up";
			ResourceName = ExpandedName;
		}
	}
	else
	{
		if (ShowAdvancedButton->IsHovered())
		{
			static const FName CollapsedHoveredName = "DetailsView.PulldownArrow.Down.Hovered";
			ResourceName = CollapsedHoveredName;
		}
		else
		{
			static const FName CollapsedName = "DetailsView.PulldownArrow.Down";
			ResourceName = CollapsedName;
		}
	}
	
	return FAppStyle::GetBrush(ResourceName);
}

EVisibility FPropertyBagInstanceDataDetails::GetAdvancedButtonVisibility() const
{
	if (HierarchyData.HierarchyRoot.IsValid())
	{
		// If no section (meaning the All section in UI) is active, we check if any property is advanced
		if (HierarchyData.ActiveHierarchySection == nullptr)
		{
			TArray<UPropertyBagHierarchyProperty*> AllHierarchyProperties;
			HierarchyData.HierarchyRoot->GetChildrenOfType<UPropertyBagHierarchyProperty>(AllHierarchyProperties, true);
			
			for(UPropertyBagHierarchyProperty* Prop : AllHierarchyProperties)
			{
				if(FPropertyBagPropertyMetadata_Common* CommonMetaData = Prop->FindPropertyMetaDataOfTypeMutable<FPropertyBagPropertyMetadata_Common>())
				{
					if(CommonMetaData->bAdvanced)
					{
						return EVisibility::Visible;
					}
				}
			}
		}
		// If there is an actual section active, we filter for categories within that section and then check their respective properties
		else
		{
			// We first collect immediate categories visible for the current section
			TArray<const UPropertyBagHierarchyCategory*> Categories;
			bool bRecursiveCategoryLoopup = false;
			HierarchyData.HierarchyRoot->GetChildrenOfType(Categories, bRecursiveCategoryLoopup);
		
			for(const UPropertyBagHierarchyCategory* Category : Categories)
			{
				const UHierarchySection* CategorySection = nullptr;
				
				// We check if they are visible under the current section
				if(const FDataHierarchyElementMetaData_SectionAssociation* SectionMetaData = Category->FindMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>())
				{
					if(SectionMetaData->Section.IsValid())
					{
						CategorySection = SectionMetaData->Section.Get();
					}
				}

				if(CategorySection != HierarchyData.ActiveHierarchySection)
				{
					continue;
				}

				// Check all properties within this category (recursively)
				TArray<UPropertyBagHierarchyProperty*> Props;
				bool bRecursivePropertyLookup = true;
				Category->GetChildrenOfType<UPropertyBagHierarchyProperty>(Props, bRecursivePropertyLookup);

				for(UPropertyBagHierarchyProperty* Prop : Props)
				{
					if(FPropertyBagPropertyMetadata_Common* CommonMetaData = Prop->FindPropertyMetaDataOfTypeMutable<FPropertyBagPropertyMetadata_Common>())
					{
						if(CommonMetaData->bAdvanced)
						{
							return EVisibility::Visible;
						}
					}
				}
			}
		}
	}
	
	return EVisibility::Collapsed;
}

////////////////////////////////////

TSharedRef<IPropertyTypeCustomization> FPropertyBagDetails::MakeInstance()
{
	return MakeShared<FPropertyBagDetails>();
}

void FPropertyBagDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropUtils = StructCustomizationUtils.GetPropertyUtilities();

	StructProperty = StructPropertyHandle;
	check(StructProperty);

	PropertyBagSchemaCDO = UE::StructUtils::ExtractPropertyBagSchemaCDO(StructPropertyHandle);
	
	if (const FProperty* MetaDataProperty = StructProperty->GetMetaDataProperty())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bFixedLayout = MetaDataProperty->HasMetaData(UE::StructUtils::Metadata::FixedLayoutName);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		bAllowContainers = MetaDataProperty->HasMetaData(UE::StructUtils::Metadata::AllowContainersName)
			? MetaDataProperty->GetBoolMetaData(UE::StructUtils::Metadata::AllowContainersName)
			: true;
		
		if (MetaDataProperty->HasMetaData(UE::StructUtils::Metadata::DefaultTypeName))
		{
			if (UEnum* Enum = StaticEnum<EPropertyBagPropertyType>())
			{
				int32 EnumIndex = Enum->GetIndexByNameString(MetaDataProperty->GetMetaData(UE::StructUtils::Metadata::DefaultTypeName));
				if (EnumIndex != INDEX_NONE)
				{
					DefaultType = EPropertyBagPropertyType(Enum->GetValueByIndex(EnumIndex));
				}
			}
		}

		// Load the feature set by the metadata set on the FPropertyBag. Can only accept explicit enum values currently.
		const FString* ChildRowFeaturesString = StructProperty->GetInstanceMetaData(UE::StructUtils::Metadata::ChildRowFeaturesName);
		if (!ChildRowFeaturesString)
		{
			ChildRowFeaturesString = MetaDataProperty->FindMetaData(UE::StructUtils::Metadata::ChildRowFeaturesName);
		}

		if (ChildRowFeaturesString)
		{
			if (const UEnum* Enum = StaticEnum<EPropertyBagChildRowFeatures>())
			{
				const int32 EnumIndex = Enum->GetIndexByNameString(*ChildRowFeaturesString);
				if (EnumIndex != INDEX_NONE)
				{
					ChildRowFeatures = static_cast<EPropertyBagChildRowFeatures>(Enum->GetValueByIndex(EnumIndex));
				}
			}
		}

		// Don't show the header if ShowOnlyInnerProperties is set
		if (MetaDataProperty->HasMetaData(UE::StructUtils::Metadata::ShowOnlyInnerPropertiesName) ||
			StructProperty->GetInstanceMetaData(UE::StructUtils::Metadata::ShowOnlyInnerPropertiesName))
		{
			return;
		}
	}

	TSharedPtr<SHorizontalBox> ValueWidget = SNew(SHorizontalBox);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!bFixedLayout && ChildRowFeatures != EPropertyBagChildRowFeatures::Fixed)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		ValueWidget->AddSlot()
		.AutoWidth()
		[
			MakeAddPropertyWidget(StructProperty, PropUtils, DefaultType).ToSharedRef()
		];
	}

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			ValueWidget.ToSharedRef()
		]
		.ShouldAutoExpand(true);
}

void FPropertyBagDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Extract the hierarchy root for this property bag (if a schema provides one)
	if (StructProperty.IsValid() && StructProperty->IsValidHandle())
	{
		HierarchyRoot = UE::StructUtils::ExtractHierarchyRoot(StructProperty.ToSharedRef());
	}

	// Create or reuse the hierarchy view model if a hierarchy root exists
	if (HierarchyRoot.IsValid() && StructProperty.IsValid() && StructProperty->IsValidHandle() && PropUtils.IsValid())
	{
		TArray<UObject*> OuterObjects;
		StructProperty->GetOuterObjects(OuterObjects);
		TSharedPtr<FPropertyPath> PropertyPath = StructProperty->CreateFPropertyPath();

		if (OuterObjects.Num() == 1 && PropertyPath.IsValid())
		{
			FStructUtilsEditorModule& Module = FModuleManager::GetModuleChecked<FStructUtilsEditorModule>("StructUtilsEditor");
			HierarchyViewModelOwner = Module.AcquireHierarchyViewModel(HierarchyRoot.Get(), OuterObjects[0], PropertyPath.ToSharedRef(), PropUtils.ToSharedRef());
		}
	}
	else
	{
		HierarchyViewModelOwner.Reset();
	}

	FPropertyBagInstanceDataDetails::FConstructParams::FHierarchyParams HierarchyParams;
	HierarchyParams.OnNavigateToHierarchyPropertyRequested = FPropertyBagInstanceDataDetails::FPropertyBagPropertyDelegate::CreateSP(this, &FPropertyBagDetails::NavigateToHierarchyProperty);
	HierarchyParams.HierarchyViewModel = HierarchyViewModelOwner.IsValid() ? HierarchyViewModelOwner->Get() : nullptr;
	FPropertyBagInstanceDataDetails::FConstructParams Params
	{
		.BagStructProperty = StructProperty,
		.PropUtils = PropUtils,
		.bAllowContainers = bAllowContainers,
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		.ChildRowFeatures = bFixedLayout ? EPropertyBagChildRowFeatures::Fixed : ChildRowFeatures,
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		.HierarchyParams = HierarchyParams
	};

	const UPropertyBagDetailsExtension* Extension = GetDefault<UPropertyBagDetailsExtension>();
	if (const FString* ExtensionClassName = StructPropertyHandle->GetInstanceMetaData(UE::StructUtils::Metadata::ExtensionClass))
	{
		if (UClass* ExtensionClass = FindObject<UClass>(nullptr, *ExtensionClassName))
		{
			if (ExtensionClass->IsChildOf(UPropertyBagDetailsExtension::StaticClass()))
			{
				if (const UPropertyBagDetailsExtension* CustomExtension = GetDefault<UPropertyBagDetailsExtension>(ExtensionClass))
				{
					Extension = CustomExtension;
				}
			}
		}
	}

	// Show the Value (FInstancedStruct) as child rows.
	const TSharedRef<FPropertyBagInstanceDataDetails> InstanceDetails = Extension->CreateInstanceDataDetails(Params);
	StructBuilder.AddCustomBuilder(InstanceDetails);
}

FReply FPropertyBagDetails::SummonHierarchyEditor(TSharedPtr<IPropertyHandle> InPropertyBagHandle, TSharedPtr<IPropertyUtilities> InPropertyUtilities)
{
	if (!InPropertyBagHandle.IsValid() || !InPropertyBagHandle->IsValidHandle() || !InPropertyUtilities.IsValid())
	{
		return FReply::Unhandled();
	}
	
	UPropertyBagHierarchyRoot* HierarchyRoot = UE::StructUtils::ExtractHierarchyRoot(InPropertyBagHandle.ToSharedRef());
	
	if (!HierarchyRoot)
	{
		return FReply::Unhandled();
	}
	
	TArray<UObject*> OuterObjects;
	InPropertyBagHandle->GetOuterObjects(OuterObjects);
	
	if (OuterObjects.Num() != 1)
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FPropertyPath> PropertyPath = InPropertyBagHandle->CreateFPropertyPath();
	if (!PropertyPath.IsValid())
	{
		return FReply::Unhandled();
	}

	FStructUtilsEditorModule& Module = FModuleManager::GetModuleChecked<FStructUtilsEditorModule>("StructUtilsEditor");

	// Fast path: if the editor is already summoned for the same UHierarchyRoot, don't rebuild the content.
	if (PropertyBagHierarchyEditorWindow.IsValid()
		&& PropertyBagHierarchyEditor.IsValid()
		&& PropertyBagHierarchyEditor.Pin()->GetHierarchyRoot() == HierarchyRoot)
	{
		// Refresh the property utilities on the existing view model so it stays bound to the
		// current details panel. The returned owner is dropped immediately — the window widget
		// already holds its own ref.
		Module.AcquireHierarchyViewModel(HierarchyRoot, OuterObjects[0], PropertyPath.ToSharedRef(), InPropertyUtilities.ToSharedRef());

		TSharedPtr<SWindow> ExistingWindow = PropertyBagHierarchyEditorWindow.Pin();
		FWindowDrawAttentionParameters Parameters;
		Parameters.RequestType = EWindowDrawAttentionRequestType::UntilActivated;
		ExistingWindow->DrawAttention(Parameters);
		bool bForce = false;
		ExistingWindow->BringToFront(bForce);
		return FReply::Handled();
	}

	bool bNewWindow = false;

	TSharedPtr<SWindow> Window;

	FString OwnerName = OuterObjects[0]->GetName();
	
	FText Title = FText::FormatOrdered(INVTEXT("{0} - {1}"), FText::FromString(OwnerName), InPropertyBagHandle->GetPropertyDisplayName());
	// We reuse the existing window if possible
	if (!PropertyBagHierarchyEditorWindow.IsValid())
	{
		Window = SNew(SWindow)
		.Title(Title)
		.ClientSize(FVector2f(750.f, 450.f));
		
		PropertyBagHierarchyEditorWindow = Window;
		bNewWindow = true;
	}
	else
	{
		Window = PropertyBagHierarchyEditorWindow.Pin();
	}
	
	Window->SetTitle(Title);

	TSharedRef<FPropertyBagHierarchyViewModelOwner> ViewModelOwner = Module.AcquireHierarchyViewModel(HierarchyRoot, OuterObjects[0], PropertyPath.ToSharedRef(), InPropertyUtilities.ToSharedRef());

	TSharedPtr<SPropertyBagHierarchyEditor> NewPropertyBagHierarchyEditor = SNew(SPropertyBagHierarchyEditor, ViewModelOwner)
	.OnCloseRequested_Static(&FPropertyBagDetails::ClosePropertyBagHierarchyEditorWindow);
	
	PropertyBagHierarchyEditor = NewPropertyBagHierarchyEditor;
	Window->SetContent(NewPropertyBagHierarchyEditor.ToSharedRef());
	
	if (bNewWindow)
	{
		FSlateApplication::Get().AddWindow(Window.ToSharedRef(), true);
	}
	else
	{
		FWindowDrawAttentionParameters Parameters;
		Parameters.RequestType = EWindowDrawAttentionRequestType::Stop;
		Window->DrawAttention(Parameters);
		Window->BringToFront(false);
	}
	
	return FReply::Handled();
}

void FPropertyBagDetails::NavigateToHierarchyProperty(const FPropertyBagPropertyDesc& Desc)
{
	if (!Desc.ID.IsValid())
	{
		return;
	}
	
	if (!PropertyBagHierarchyEditor.IsValid() || !PropertyBagHierarchyEditor.Pin()->GetPropertyBagHandle().IsValid() || !PropertyBagHierarchyEditor.Pin()->GetPropertyBagHandle()->IsSamePropertyNode(StructProperty))
	{
		SummonHierarchyEditor(StructProperty.ToSharedRef(), PropUtils.ToSharedRef());
	}
	
	if (PropertyBagHierarchyEditor.IsValid())
	{
		PropertyBagHierarchyEditor.Pin()->NavigateToProperty(Desc, true);
	}
}

void FPropertyBagDetails::ClosePropertyBagHierarchyEditorWindow()
{
	if (PropertyBagHierarchyEditorWindow.IsValid())
	{
		PropertyBagHierarchyEditorWindow.Pin()->RequestDestroyWindow();
	}
}

TSharedPtr<SWidget> FPropertyBagDetails::MakeAddPropertyWidget(TSharedPtr<IPropertyHandle> InStructProperty, TSharedPtr<IPropertyUtilities> InPropUtils, EPropertyBagPropertyType DefaultType, const FSlateColor IconColor)
{
	const FText RemoveAllPropertiesText = LOCTEXT("OnRemoveAllProperties", "Remove all Properties");

	TSharedRef<SHorizontalBox> Container = SNew(SHorizontalBox)
	+SHorizontalBox::Slot()
	.AutoWidth()
	[
		SNew(SButton)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ToolTipText(LOCTEXT("AddProperty_Tooltip", "Add new property"))
		.OnClicked_Lambda([StructProperty = InStructProperty, PropUtils = InPropUtils, DefaultType]()
		{
			constexpr int32 MaxIterations = 100;
			FName NewName(TEXT("NewProperty"));
			int32 Number = 1;
			while (!UE::StructUtils::Private::IsUniqueName(NewName, FName(), StructProperty) && Number < MaxIterations)
			{
				Number++;
				NewName.SetNumber(Number);
			}
			if (Number == MaxIterations)
			{
				return FReply::Handled();
			}

			UE::StructUtils::ApplyChangesToPropertyDescs(
				LOCTEXT("OnPropertyAdded", "Add Property"), StructProperty,
				[&NewName, DefaultType](TArray<FPropertyBagPropertyDesc>& PropertyDescs)
				{
					PropertyDescs.Emplace(NewName, DefaultType);
				});

			return FReply::Handled();

		})
		.Content()
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
			.ColorAndOpacity(IconColor)
		]
	]
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
	[
		PropertyCustomizationHelpers::MakeEmptyButton(
			FSimpleDelegate::CreateLambda([StructProperty = InStructProperty, PropUtils = InPropUtils, RemoveAllPropertiesText]()
				{
					UE::StructUtils::ApplyChangesToPropertyDescs(
						RemoveAllPropertiesText,
						StructProperty,
						[](TArray<FPropertyBagPropertyDesc>& PropertyDescs)
						{
							PropertyDescs.Empty();
						});
				}),
			RemoveAllPropertiesText
			)
	];
	
	// We attempt to retrieve a hierarchy root via schema, and display the Edit Hierarchy button if successful
	if (UPropertyBagHierarchyRoot* Root = UE::StructUtils::ExtractHierarchyRoot(InStructProperty.ToSharedRef()))
	{		
		Container->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("EditHierarchy_Tooltip", "Summon Hierarchy Editor to edit categories and metadata."))
			.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
			.OnClicked_Static(&SummonHierarchyEditor, InStructProperty, InPropUtils)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.EditHierarchy"))
				.ColorAndOpacity(IconColor)
			]
		];
	}
	
	return Container;
}

////////////////////////////////////
bool UPropertyBagSchema::SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType, const EPinContainerType& ContainerType) const
{
	return ContainerType == EPinContainerType::None || ContainerType == EPinContainerType::Array || ContainerType == EPinContainerType::Set || ContainerType == EPinContainerType::Map;
}

UPropertyBagHierarchyRoot* UPropertyBagSchema::GetHierarchyRoot(const TArray<UObject*>& ObjectsWithProperty) const
{
	return nullptr;
}

TArray<UScriptStruct*> UPropertyBagSchema::GetHierarchyPropertyMetaDataTypes(const FPropertyBagPropertyDesc& Desc) const
{
	TArray<UScriptStruct*> Result;
	
	Result.Add(FPropertyBagPropertyMetadata_Common::StaticStruct());
	
	if (Desc.IsNumericType() && Desc.ValueType != EPropertyBagPropertyType::Bool && Desc.ContainerTypes.IsEmpty())
	{			
		Result.Add(FPropertyBagPropertyMetadata_Numeric::StaticStruct());
	}
	
	return Result;
}

TSubclassOf<UPropertyBagHierarchyCategory> UPropertyBagSchema::GetHierarchyCategoryType() const
{
	return UPropertyBagHierarchyCategory::StaticClass();
}

TSubclassOf<UPropertyBagHierarchySection> UPropertyBagSchema::GetHierarchySectionType() const
{
	return UPropertyBagHierarchySection::StaticClass();
}

void UPropertyBagSchema::TransferPropertyBagMetadataIntoHierarchy(const FInstancedPropertyBag& PropertyBag,	UPropertyBagHierarchyRoot& HierarchyRoot) const
{
	if (PropertyBag.GetPropertyBagStruct() == nullptr)
	{
		return;
	}
	
	TMap<const FPropertyBagPropertyDesc*, TArray<FString>> PropertyCategoryMap;
	
	for (const FPropertyBagPropertyDesc& Desc : PropertyBag.GetPropertyBagStruct()->GetPropertyDescs())
	{
		TArray<FString>& Array = PropertyCategoryMap.Add(&Desc);
		
		FString CategoryString = Desc.GetMetaData(UE::StructUtils::Metadata::CategoryName);
		TArray<FString> CategoriesOfDesc;
		CategoryString.ParseIntoArray(CategoriesOfDesc, TEXT("|"), true);
		
		if (CategoriesOfDesc.Num() > 0)
		{	
			Array.Append(CategoriesOfDesc);
		}
	}
	
	auto GenerateHierarchyRecursive = [&](auto&& Self, const FPropertyBagPropertyDesc& Desc, const TArray<FString>& CategoryChain, UHierarchyElement* Outer) -> void
	{
		if (CategoryChain.Num() > 0)
		{				
			TArray<UPropertyBagHierarchyCategory*> ExistingCategories;
			Outer->GetChildrenOfType<UPropertyBagHierarchyCategory>(ExistingCategories, false);
				
			FString CurrentCategory = CategoryChain[0];
			
			UPropertyBagHierarchyCategory* HierarchyCategory = nullptr;
		
			UPropertyBagHierarchyCategory** ExistingHierarchyCategory = ExistingCategories.FindByPredicate([CurrentCategory](UPropertyBagHierarchyCategory* Candidate)
			{
				return Candidate->GetCategoryName().ToString().Equals(CurrentCategory);
			});
		
			if (ExistingHierarchyCategory)
			{
				HierarchyCategory = *ExistingHierarchyCategory;
			}
			else
			{
				HierarchyCategory = NewObject<UPropertyBagHierarchyCategory>(Outer, GetHierarchyCategoryType());
				HierarchyCategory->SetCategoryName(FName(CurrentCategory));
				Outer->GetChildrenMutable().Add(HierarchyCategory);
			}
			
			TArray<FString> ChildCategoryChain = CategoryChain;
			ChildCategoryChain.RemoveAt(0);
			Self(Self, Desc, ChildCategoryChain, HierarchyCategory);
		}
		else
		{
			TArray<UScriptStruct*> PropertyMetaDataTypes = GetHierarchyPropertyMetaDataTypes(Desc);
			UPropertyBagHierarchyProperty* HierarchyProperty = NewObject<UPropertyBagHierarchyProperty>(Outer);
			HierarchyProperty->Initialize(Desc);
			Outer->GetChildrenMutable().Add(HierarchyProperty);
			
			for (UScriptStruct* MetaDataScriptStruct : PropertyMetaDataTypes)
			{
				HierarchyProperty->PropertyMetadata.Values.Add(FInstancedStruct(MetaDataScriptStruct));
			}
			
			if (FPropertyBagPropertyMetadata_Common* CommonMetaData = HierarchyProperty->FindPropertyMetaDataOfTypeMutable<FPropertyBagPropertyMetadata_Common>())
			{
				if (Desc.HasMetaData(UE::StructUtils::Metadata::Specifiers::ToolTipName))
				{
					CommonMetaData->Tooltip = FText::FromString(Desc.GetMetaData(UE::StructUtils::Metadata::Specifiers::ToolTipName));
				}
			}
			
			if (FPropertyBagPropertyMetadata_Numeric* NumericMetaData = HierarchyProperty->FindPropertyMetaDataOfTypeMutable<FPropertyBagPropertyMetadata_Numeric>())
			{
				if (Desc.HasMetaData(UE::StructUtils::Metadata::Specifiers::ClampMinName))
				{
					float ClampMinValue;
					if (LexTryParseString(ClampMinValue, *Desc.GetMetaData(UE::StructUtils::Metadata::Specifiers::ClampMinName)))
					{
						NumericMetaData->bUseClampMin = true;
						NumericMetaData->ClampMin = ClampMinValue;	
					}
				}
				
				if (Desc.HasMetaData(UE::StructUtils::Metadata::Specifiers::ClampMaxName))
				{
					float ClampMaxValue;
					if (LexTryParseString(ClampMaxValue, *Desc.GetMetaData(UE::StructUtils::Metadata::Specifiers::ClampMaxName)))
					{
						NumericMetaData->bUseClampMax = true;
						NumericMetaData->ClampMax = ClampMaxValue;	
					}
				}
				
				if (Desc.HasMetaData(UE::StructUtils::Metadata::Specifiers::UIMinName))
				{
					float UIMinValue;
					if (LexTryParseString(UIMinValue, *Desc.GetMetaData(UE::StructUtils::Metadata::Specifiers::UIMinName)))
					{
						NumericMetaData->bUseUIMin = true;
						NumericMetaData->UIMin = UIMinValue;	
					}
				}
				
				if (Desc.HasMetaData(UE::StructUtils::Metadata::Specifiers::UIMaxName))
				{
					float UIMaxValue;
					if (LexTryParseString(UIMaxValue, *Desc.GetMetaData(UE::StructUtils::Metadata::Specifiers::UIMaxName)))
					{
						NumericMetaData->bUseUIMax = true;
						NumericMetaData->UIMax = UIMaxValue;	
					}
				}
				
				if (Desc.HasMetaData(UE::StructUtils::Metadata::Specifiers::UnitsName))
				{
					FString UnitString = Desc.GetMetaData(UE::StructUtils::Metadata::Specifiers::UnitsName);
					if (TOptional<EUnit> Unit = FUnitConversion::UnitFromString(*UnitString))
					{
						NumericMetaData->bUseUnit = true;
						NumericMetaData->Unit = Unit.GetValue();
					}
				}
			}
		}
	};
	
	for (const auto& PropertyData : PropertyCategoryMap)
	{
		GenerateHierarchyRecursive(GenerateHierarchyRecursive, *PropertyData.Key, PropertyData.Value, &HierarchyRoot);
	}
}

#undef LOCTEXT_NAMESPACE
