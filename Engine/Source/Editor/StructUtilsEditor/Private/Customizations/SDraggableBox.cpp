// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDraggableBox.h"

#include "IDetailDragDropHandler.h"
#include "Framework/Application/SlateApplication.h"

namespace StructUtilsEditor
{
	void SDraggableBox::Construct(const FArguments& InArgs)
	{
		DragDropHandler = InArgs._DragDropHandler;
		bRequireDirectHover = InArgs._RequireDirectHover;

		ChildSlot
		[
			InArgs._Content.Widget
		];
	}

	FReply SDraggableBox::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		return FReply::Handled().DetectDrag(SharedThis<SWidget>(this), EKeys::LeftMouseButton);
	}

	FReply SDraggableBox::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (!bRequireDirectHover || IsDirectlyHovered())
		{
			if (DragDropHandler && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
			{
				const TSharedPtr<FDragDropOperation> DragDropOp = DragDropHandler->CreateDragDropOperation();
				if (DragDropOp.IsValid())
				{
					return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
				}
			}
		}

		return FReply::Unhandled();
	}

	FCursorReply SDraggableBox::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
	{
		if (!bRequireDirectHover || IsDirectlyHovered())
		{
			return CursorEvent.IsMouseButtonDown(EKeys::LeftMouseButton) ? FCursorReply::Cursor(EMouseCursor::GrabHandClosed) : FCursorReply::Cursor(EMouseCursor::GrabHand);
		}
		else
		{
			return FCursorReply::Unhandled();
		}
	}
}