// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class IDetailDragDropHandler;

namespace StructUtilsEditor
{
	class SDraggableBox : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDraggableBox) {}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ARGUMENT(TSharedPtr<IDetailDragDropHandler>, DragDropHandler)
		SLATE_ARGUMENT(bool, RequireDirectHover)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

		//~Begin SWidget interface
		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
		//~End SWidget interface

	protected:
		TSharedPtr<IDetailDragDropHandler> DragDropHandler = nullptr;
		bool bRequireDirectHover = true;
	};
}