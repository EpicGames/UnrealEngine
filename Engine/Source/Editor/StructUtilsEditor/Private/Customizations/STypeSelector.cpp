// Copyright Epic Games, Inc. All Rights Reserved.

#include "STypeSelector.h"

#include "PinTypeSelectorFilter.h"
#include "Framework/Application/SlateApplication.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/WidgetPath.h"

#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

/** Exact replication from SPinTypeSelector, which is not exposed. */
class FPinTypeSelectorCustomFilterProxy final : public IPinTypeSelectorFilter
{
public:
	FPinTypeSelectorCustomFilterProxy(TSharedRef<IPinTypeSelectorFilter> InFilter, FSimpleDelegate InOnFilterChanged)
		: Filter(InFilter)
	{
		// Auto-register the given delegate to respond to any filter change event and refresh the filtered item list, etc.
		OnFilterChanged_DelegateHandle = Filter->RegisterOnFilterChanged(InOnFilterChanged);
	}

	~FPinTypeSelectorCustomFilterProxy() override
	{
		// Auto-unregister the delegate that was previously registered at construction time.
		Filter->UnregisterOnFilterChanged(OnFilterChanged_DelegateHandle);
	}

	virtual FDelegateHandle RegisterOnFilterChanged(const FSimpleDelegate InOnFilterChanged) override
	{
		return Filter->RegisterOnFilterChanged(InOnFilterChanged);
	}

	virtual void UnregisterOnFilterChanged(const FDelegateHandle InDelegateHandle) override
	{
		Filter->UnregisterOnFilterChanged(InDelegateHandle);
	}

	virtual TSharedPtr<SWidget> GetFilterOptionsWidget() override
	{
		return Filter->GetFilterOptionsWidget();
	}

	virtual bool ShouldShowPinTypeTreeItem(const FPinTypeTreeItem InItem) const override
	{
		return Filter->ShouldShowPinTypeTreeItem(InItem);
	}

private:
	/** The underlying filter for which we're acting as a proxy. */
	TSharedRef<IPinTypeSelectorFilter> Filter;

	/** A handle to a delegate that gets called whenever the custom filter changes. Will be unregistered automatically when the proxy is destroyed. */
	FDelegateHandle OnFilterChanged_DelegateHandle;
};

void STypeSelector::Construct(const FArguments& InArgs, FGetPinTypeTree GetPinTypeTreeFunc)
{
	// Currently only wrapping around the Compact selector type. All others should pass to the SPinTypeSelector.
	if (InArgs._SelectorType != ESelectorType::Compact)
	{
		// Forward all arguments, construct, and return.
		SPinTypeSelector::FArguments ParentArgs;
		ParentArgs._TargetPinType = InArgs._TargetPinType;
		ParentArgs._Schema = InArgs._Schema;
		ParentArgs._SchemaAction = InArgs._SchemaAction;
		ParentArgs._TypeTreeFilter = InArgs._TypeTreeFilter;
		ParentArgs._bAllowArrays = InArgs._bAllowContainers;
		ParentArgs._TreeViewWidth = InArgs._TreeViewWidth;
		ParentArgs._TreeViewHeight = InArgs._TreeViewHeight;
		ParentArgs._OnPinTypePreChanged = InArgs._OnPinTypePreChanged;
		ParentArgs._OnPinTypeChanged = InArgs._OnPinTypeChanged;
		ParentArgs._SelectorType = InArgs._SelectorType;
		ParentArgs._ReadOnly = InArgs._ReadOnly;
		ParentArgs._CustomFilters = InArgs._CustomFilters;
		ParentArgs._OnPinTypeChanged = InArgs._OnPinTypeChanged;
		SPinTypeSelector::Construct(ParentArgs, GetPinTypeTreeFunc);
		return;
	}

	SearchText = FText::GetEmpty();

	ReadOnly = InArgs._ReadOnly;

	OnTypeChanged = InArgs._OnPinTypeChanged;
	OnTypePreChanged = InArgs._OnPinTypePreChanged;

	check(GetPinTypeTreeFunc.IsBound());
	GetPinTypeTree = GetPinTypeTreeFunc;

	Schema = InArgs._Schema;
	SchemaAction = InArgs._SchemaAction;
	TypeTreeFilter = InArgs._TypeTreeFilter;
	TreeViewWidth = InArgs._TreeViewWidth;
	TreeViewHeight = InArgs._TreeViewHeight;

	TargetPinType = InArgs._TargetPinType;
	SelectorType = InArgs._SelectorType;

	NumFilteredPinTypeItems = 0;
	NumValidPinTypeItems = 0;

	bIsRightMousePressed = false;

	if (InArgs._CustomFilters.Num() > 0)
	{
		for (const TSharedPtr<IPinTypeSelectorFilter>& Filter : InArgs._CustomFilters)
		{
			CustomFilters.Add(MakeShared<FPinTypeSelectorCustomFilterProxy>(Filter.ToSharedRef(), FSimpleDelegate::CreateSP(this, &STypeSelector::OnCustomFilterChanged)));
		}
	}
	else if (UClass* PinTypeSelectorFilterClass = GetDefault<UPinTypeSelectorFilter>()->FilterClass.LoadSynchronous())
	{
		const TSharedPtr<IPinTypeSelectorFilter> SelectorFilter = GetDefault<UPinTypeSelectorFilter>(PinTypeSelectorFilterClass)->GetPinTypeSelectorFilter();
		CustomFilters.Add(MakeShared<FPinTypeSelectorCustomFilterProxy>(SelectorFilter.ToSharedRef(), FSimpleDelegate::CreateSP(this, &STypeSelector::OnCustomFilterChanged)));
	}

	const TSharedPtr<SWidget> ReadOnlyWidget = SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(2.0f, 3.0f, 2.0f, 3.0f))
		.AutoWidth()
		[
			// Read-only version does not display container or secondary type separately, so we need to jam it all in the one image
			SNew(SLayeredImage, TAttribute<const FSlateBrush*>(this, &STypeSelector::GetSecondaryTypeIconImage), TAttribute<FSlateColor>(this, &STypeSelector::GetSecondaryTypeIconColor))
			.Image(this, &STypeSelector::GetTypeIconImage)
			.ColorAndOpacity(this, &STypeSelector::GetTypeIconColor)
		]
		+ SHorizontalBox::Slot()
		.Padding(2.0f, 2.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(this, &STypeSelector::GetTypeDescription, false)
			.Font(InArgs._Font)
			.ColorAndOpacity(FSlateColor::UseForeground())
		];

	SAssignNew(TypeComboButton, SComboButton)
	.OnGetMenuContent(this, &STypeSelector::GetMenuContent, false)
	.ContentPadding(0.0f)
	.ToolTipText(this, &STypeSelector::GetToolTipForSelector)
	.HasDownArrow(false)
	.ButtonStyle(FAppStyle::Get(), "SimpleButton")
	.ButtonContent()
	[
		SNew(SLayeredImage,
			TAttribute<const FSlateBrush*>(this, &STypeSelector::GetSecondaryTypeIconImage),
			TAttribute<FSlateColor>(this, &STypeSelector::GetSecondaryTypeIconColor))
		.Image(this, &STypeSelector::GetTypeIconImage)
		.ColorAndOpacity(this, &STypeSelector::GetTypeIconColor)
	];

	this->ChildSlot
	[
		SNew(SWidgetSwitcher)
		.WidgetIndex_Lambda([this]() { return ReadOnly.Get() ? 1 : 0; })
		+ SWidgetSwitcher::Slot() // editable version
		.Padding(-6.0f, 0.0f, 0.0f, 0.0f)
		[
			TypeComboButton.ToSharedRef()
		]
		+ SWidgetSwitcher::Slot() // read-only version
		[
			ReadOnlyWidget.ToSharedRef()
		]
	];
}

FReply STypeSelector::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		bIsRightMousePressed = true;
	}

	return FReply::Unhandled();
}

FReply STypeSelector::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && bIsRightMousePressed)
	{
		// Push the other menu from the secondary ComboButton for the type.
		FSlateApplication::Get().PushMenu(TypeComboButton.ToSharedRef(),
			FWidgetPath(),
			GetPinContainerTypeMenuContent(),
			MyGeometry.AbsolutePosition,
			FPopupTransitionEffect(FPopupTransitionEffect::SubMenu));

		bIsRightMousePressed = false;
		return FReply::Handled();
	}
	else
	{
		bIsRightMousePressed = false;
		return FReply::Unhandled();
	}
}

TSharedRef<SWidget> STypeSelector::GetMenuContent(const bool bForSecondaryType)
{
	// Reset the TypeTreeRoot in case property type filters or funcs have changed.
	TypeTreeRoot.Reset();
	return SPinTypeSelector::GetMenuContent(bForSecondaryType);
}

FText STypeSelector::GetToolTipForSelector() const
{
	FText TooltipText;
	if (IsEnabled())
	{
		TooltipText = NSLOCTEXT("STypeSelector", "PinTypeSelectorTooltip", "Left click to select the variable's type. Right click to select a container type.");
	}
	else
	{
		TooltipText = NSLOCTEXT("STypeSelector", "DisabledPinTypeSelectorTooltip", "Cannot edit variable type when they are inherited from parent.");
	}

	return FText::Format(INVTEXT("Type: {0}\n{1}"), GetTypeDescription(/*bIncludeSubcategory=*/false), TooltipText);
}
