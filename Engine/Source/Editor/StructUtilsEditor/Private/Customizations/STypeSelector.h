// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SPinTypeSelector.h"

#define UE_API STRUCTUTILSEDITOR_API

/**
 * This widget is a small wrapper around SPinTypeSelector to allow for a right click context menu on the selector pill
 * for the container type so that it can all fit into one compact combo button.
 * 
 * Eventually (TODO), it should be replaced by a base combo button drop down that supports right clicks as
 * well as an update to the original SPinTypeSelector to support this for Compact Selector Type (rather than toggling).
 * However, this will require updates to core widgets, which are currently hardcoded for left click only.
 * 
 * Ex. 'SDoubleComboButton' (SComboButton) which inherits from 'SDoubleMenuAnchor' (SMenuAnchor) or similar with two
 * menu anchors. */
UE_EXPERIMENTAL(5.6, "This widget is an experimental prototype for StructUtils use only.")
class STypeSelector final : public SPinTypeSelector
{
public:
	// Replicated arguments of SPinTypeSelector
	SLATE_BEGIN_ARGS(STypeSelector)
			: _Schema(nullptr)
			, _SchemaAction(nullptr)
			, _TypeTreeFilter(ETypeTreeFilter::None)
			, _bAllowContainers(true)
			, _TreeViewWidth(300.f)
			, _TreeViewHeight(350.f)
			, _Font(FAppStyle::GetFontStyle(TEXT("NormalFont")))
			, _SelectorType(ESelectorType::Full)
			, _ReadOnly(false) {}

		SLATE_ATTRIBUTE(FEdGraphPinType, TargetPinType)
		SLATE_ARGUMENT(const UEdGraphSchema*, Schema)
		SLATE_ARGUMENT(TWeakPtr<const FEdGraphSchemaAction>, SchemaAction)
		SLATE_ARGUMENT(ETypeTreeFilter, TypeTreeFilter)
		SLATE_ARGUMENT(bool, bAllowContainers)
		SLATE_ATTRIBUTE(FOptionalSize, TreeViewWidth)
		SLATE_ATTRIBUTE(FOptionalSize, TreeViewHeight)
		SLATE_EVENT(FOnPinTypeChanged, OnPinTypePreChanged)
		SLATE_EVENT(FOnPinTypeChanged, OnPinTypeChanged)
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		SLATE_ARGUMENT(ESelectorType, SelectorType)
		SLATE_ATTRIBUTE(bool, ReadOnly)
		SLATE_ARGUMENT(TArray<TSharedPtr<class IPinTypeSelectorFilter>>, CustomFilters)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, FGetPinTypeTree GetPinTypeTreeFunc);

	// SWidget interface
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	// End of SWidget interface

	UE_API virtual TSharedRef<SWidget> GetMenuContent(bool bForSecondaryType) override;

private:
	UE_API FText GetToolTipForSelector() const;
};

#undef UE_API
