// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyEditor/PropertyBagHierarchyViewModel.h"

#include "IDetailTreeNode.h"
#include "IPropertyUtilities.h"
#include "PropertyBagDetails.h"
#include "PropertyHandle.h"
#include "PropertyPath.h"
#include "StructUtilsEditorUtilsPrivate.h"
#include "StructUtilsMetadata.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Modules/ModuleManager.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "PropertyBagHierarchyEditor"

void FPropertyBagPropertyMetadata_Common::PostEditChange(const FPropertyChangedEvent& PropertyChangedEvent,	TSharedPtr<FPropertyBagHierarchyPropertyViewModel> PropertyViewModel)
{
	if (!PropertyViewModel.IsValid())
	{
		return;
	}
	UPropertyBagHierarchyViewModel* ViewModel = Cast<UPropertyBagHierarchyViewModel>(PropertyViewModel->GetHierarchyViewModel());
	
	if (PropertyChangedEvent.Property)
	{
		const FPropertyBagPropertyDesc* PropertyDesc = PropertyViewModel->GetPropertyDesc();
		
		if (PropertyDesc == nullptr)
		{
			return;
		}
		
		
		if (PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(FPropertyBagPropertyMetadata_Common, Tooltip))
		{
			UE::StructUtils::ApplyChangesToSinglePropertyDesc(
				LOCTEXT("ChangedTooltip", "Changed tooltip"),
				*PropertyDesc,
				ViewModel->GetPropertyBagHandle(),
				[Tooltip = Tooltip](FPropertyBagPropertyDesc& Desc)
					{
						// If empty, remove the specifier
						if (Tooltip.IsEmpty())
						{
							Desc.RemoveMetadata(UE::StructUtils::Metadata::Specifiers::ToolTipName);
						}
						else
						{
							Desc.SetMetaData(UE::StructUtils::Metadata::Specifiers::ToolTipName, Tooltip.ToString());
						}
					});
		}
		
		if (PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(FPropertyBagPropertyMetadata_Common, bAdvanced))
		{
			UE::StructUtils::ApplyChangesToSinglePropertyDesc(
				LOCTEXT("ToggleAdvanced", "Toggled advance"),
				*PropertyDesc,
				ViewModel->GetPropertyBagHandle(),
				[bInAdvanced = bAdvanced](FPropertyBagPropertyDesc& Desc)
					{
						if (bInAdvanced)
						{
							Desc.PropertyFlags |= CPF_AdvancedDisplay;
						}
						else
						{
							Desc.PropertyFlags &= ~CPF_AdvancedDisplay;
						}
					});
		}
	}
}

void FPropertyBagPropertyMetadata_Numeric::PostEditChange(const FPropertyChangedEvent& PropertyChangedEvent, TSharedPtr<FPropertyBagHierarchyPropertyViewModel> PropertyViewModel)
{
	using namespace UE::StructUtils::Metadata::Specifiers;
	
	if (!PropertyViewModel.IsValid())
	{
		return;
	}
	
	UPropertyBagHierarchyViewModel* ViewModel = Cast<UPropertyBagHierarchyViewModel>(PropertyViewModel->GetHierarchyViewModel());
	
	if (PropertyChangedEvent.Property)
	{
		const FPropertyBagPropertyDesc* PropertyDesc = PropertyViewModel->GetPropertyDesc();
		
		if (PropertyDesc == nullptr)
		{
			return;
		}
		
		// Struct used for the transaction session name for different property type.
		struct SessionName
		{
			FText UsePropSessionName;
			FText PropSessionName;
		};
		SessionName ClampMinSessionName{LOCTEXT("ChangedUseClampMin", "Changed Use Clamp Min"), LOCTEXT("ChangedClampMin", "Changed Clamp Min")};
		SessionName ClampMaxSessionName{LOCTEXT("ChangedUseClampMax", "Changed Use Clamp Max"), LOCTEXT("ChangedClampMax", "Changed Clamp Max")};
		SessionName UIMinSessionName{LOCTEXT("ChangedUseUIMin", "Changed Use UI Min"), LOCTEXT("ChangedUIMin", "Changed UI Min")};
		SessionName UIMaxSessionName{LOCTEXT("ChangedUseUIMax", "Changed Use UI Max"), LOCTEXT("ChangedUIMax", "Changed UI Max")};
		SessionName UnitSessionName{LOCTEXT("ChangedUseUnit", "Changed Use Unit"), LOCTEXT("ChangedUnit", "Changed Unit")};
		
		// Generic function to treat a property change event. 
		TSharedPtr<IPropertyHandle> PropertyHandle = ViewModel->GetPropertyBagHandle();
		auto TreatPropertyChange = [ChangeType = PropertyChangedEvent.ChangeType, &PropertyHandle, PropertyDesc, this]<typename PropType>(const SessionName& InSessionName, bool bUsePropValue, PropType (FPropertyBagPropertyMetadata_Numeric::*PropMemberPtr) , const PropType& PropDefaultValue, const TArray<FName>& PropNames, const auto& SanitizePropertyFunc)
			{
				static_assert(std::is_invocable_v<decltype(SanitizePropertyFunc), PropType>);
				static_assert(std::is_same_v<decltype(SanitizePropertyFunc(std::declval<PropType>())), FString>);

				// Case where the user toggle the UseCustomValue checkbox.
				if (ChangeType == EPropertyChangeType::ToggleEditable)
				{
					// Back to default value if we are not using a custom value.
					if (!bUsePropValue)
					{
						this->*PropMemberPtr = PropDefaultValue;
					}
					UE::StructUtils::ApplyChangesToSinglePropertyDesc(
						InSessionName.UsePropSessionName,
						*PropertyDesc,
						PropertyHandle,
						[&SanitizePropertyFunc, bUsePropValue, PropValue = this->*PropMemberPtr, &PropNames](FPropertyBagPropertyDesc& Desc)
							{
								if (bUsePropValue)
								{
									for (FName PropName : PropNames)
									{
										Desc.SetMetaData(PropName, SanitizePropertyFunc(PropValue));
									}
								}
								else
								{
									for (FName PropName : PropNames)
									{
										Desc.RemoveMetadata(PropName);
									}
								}
							});
				}
				else // Modified the value per se.
				{
					UE::StructUtils::ApplyChangesToSinglePropertyDesc(
						InSessionName.PropSessionName,
						*PropertyDesc,
						PropertyHandle,
						[&SanitizePropertyFunc, PropValue = this->*PropMemberPtr, &PropNames](FPropertyBagPropertyDesc& Desc)
							{
								for (FName PropName : PropNames)
								{
									Desc.SetMetaData(PropName, SanitizePropertyFunc(PropValue));
								}
							});
				}
			};

		auto SanitizeFloatFunc = [](float InFloat) -> FString {return FString::SanitizeFloat(InFloat);};
		auto SanitizeUnitFunc = [](EUnit InUnit) -> FString {return FUnitConversion::GetUnitDisplayString(InUnit);};
		if (PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(FPropertyBagPropertyMetadata_Numeric, ClampMin))
		{
			TreatPropertyChange(ClampMinSessionName, 
				bUseClampMin, 
				&FPropertyBagPropertyMetadata_Numeric::ClampMin, 
				DefaultClampMin,
				{ClampMinName},
				SanitizeFloatFunc);
		}

		if (PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(FPropertyBagPropertyMetadata_Numeric, ClampMax))
		{
			TreatPropertyChange(ClampMaxSessionName, 
				bUseClampMax, 
				&FPropertyBagPropertyMetadata_Numeric::ClampMax,
				DefaultClampMax,
				{ClampMaxName},
				SanitizeFloatFunc);
		}

		if (PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(FPropertyBagPropertyMetadata_Numeric, UIMin))
		{
			TreatPropertyChange(UIMinSessionName, 
				bUseUIMin, 
				&FPropertyBagPropertyMetadata_Numeric::UIMin, 
				DefaultUIMin,
				{UIMinName},
				SanitizeFloatFunc);
		}

		if (PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(FPropertyBagPropertyMetadata_Numeric, UIMax))
		{
			TreatPropertyChange(UIMaxSessionName, 
				bUseUIMax, 
				&FPropertyBagPropertyMetadata_Numeric::UIMax,
				DefaultUIMax,
				{UIMaxName},
				SanitizeFloatFunc);
		}

		if (PropertyChangedEvent.Property->GetName() == GET_MEMBER_NAME_CHECKED(FPropertyBagPropertyMetadata_Numeric, Unit))
		{
			TreatPropertyChange(UnitSessionName, 
				bUseUnit, 
				&FPropertyBagPropertyMetadata_Numeric::Unit,
				DefaultUnit,
				{UnitsName, ForceUnitsName},
				SanitizeUnitFunc);
		}
	}
}

void UPropertyBagHierarchyProperty::Initialize(const FPropertyBagPropertyDesc& InPropertyDesc)
{
	SetIdentity(ConstructIdentity(InPropertyDesc));
}

UPropertyBagHierarchyProperty* UPropertyBagHierarchyProperty::GetArchetype() const
{
	// TODO: We could hold a cache of the archetype, but it would need to make PropertyMetadata private to be consistent.
	UPropertyBagHierarchyProperty* PropertyBagHierarchyProperty = NewObject<UPropertyBagHierarchyProperty>(GetTransientPackageAsObject(), NAME_None, RF_ArchetypeObject);
	
	for (const FInstancedStruct& MetaDataElement : PropertyMetadata.Values)
	{
		PropertyBagHierarchyProperty->FindOrAddPropertyMetaData(MetaDataElement.GetScriptStruct());
	}
	return PropertyBagHierarchyProperty;
}

FHierarchyElementIdentity UPropertyBagHierarchyProperty::ConstructIdentity(const FPropertyBagPropertyDesc& PropertyDesc)
{
	return FHierarchyElementIdentity({PropertyDesc.ID}, {});
}

FGuid UPropertyBagHierarchyProperty::GetPropertyId() const
{
	if (Identity.Guids.Num() == 1)
	{
		return Identity.Guids[0];
	}
	
	return FGuid();
}

FInstancedStruct* UPropertyBagHierarchyProperty::FindOrAddPropertyMetaData(const UScriptStruct* InScriptStruct)
{
	if (!ContainsPropertyMetaDataOfType(InScriptStruct))
	{
		return &PropertyMetadata.Values.Add_GetRef(FInstancedStruct(InScriptStruct));
	}
	
	return FindPropertyMetaDataOfTypeMutable(InScriptStruct);
}

bool UPropertyBagHierarchyProperty::ContainsPropertyMetaDataOfType(const UScriptStruct* InScriptStruct)
{		
	FInstancedStruct* FoundStruct = PropertyMetadata.Values.FindByPredicate([InScriptStruct](const FInstancedStruct& Candidate)
	{
		return Candidate.GetScriptStruct() == InScriptStruct;
	});
	
	return FoundStruct ? true : false;
}

const FInstancedStruct* UPropertyBagHierarchyProperty::FindPropertyMetaDataOfType(const UScriptStruct* InScriptStruct) const
{		
	const FInstancedStruct* FoundStruct = PropertyMetadata.Values.FindByPredicate([InScriptStruct](const FInstancedStruct& Candidate)
	{
		return Candidate.GetScriptStruct() == InScriptStruct;
	});
		
	if (FoundStruct)
	{
		return FoundStruct;
	}
		
	return nullptr;
}

FInstancedStruct* UPropertyBagHierarchyProperty::FindPropertyMetaDataOfTypeMutable(const UScriptStruct* InScriptStruct)
{
	FInstancedStruct* FoundStruct = PropertyMetadata.Values.FindByPredicate([InScriptStruct](const FInstancedStruct& Candidate)
	{
		return Candidate.GetScriptStruct() == InScriptStruct;
	});
		
	if (FoundStruct)
	{
		return FoundStruct;
	}
		
	return nullptr;
}

FPropertyBagHierarchyPropertyViewModel::FPropertyBagHierarchyPropertyViewModel(UPropertyBagHierarchyProperty* InProperty, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel)
	: FHierarchyItemViewModel(InProperty, InParent, InHierarchyViewModel)
{
	
}

void FPropertyBagHierarchyPropertyViewModel::UpdateMetaData()
{
	const FPropertyBagPropertyDesc* Desc = GetPropertyDesc();
	if (Desc == nullptr)
	{
		return;
	}
	
	TSharedPtr<IPropertyHandle> PropertyBagHandle = Cast<UPropertyBagHierarchyViewModel>(HierarchyViewModel.Get())->GetPropertyBagHandle();
	
	// If the handle is not valid, we assume the data still exists. Handle might have just become invalidated.
	if (!PropertyBagHandle.IsValid() || !PropertyBagHandle->IsValidHandle())
	{
		return;
	}
	
	// We remove metadata that the schema says is not compatible, and add metadata that is.
	if (const UPropertyBagSchema* PropertyBagSchemaCDO = UE::StructUtils::ExtractPropertyBagSchemaCDO(PropertyBagHandle.ToSharedRef()))
	{
		TArray<UScriptStruct*> AvailableMetaDataTypes = PropertyBagSchemaCDO->GetHierarchyPropertyMetaDataTypes(*Desc);
		
		GetDataMutable<UPropertyBagHierarchyProperty>()->PropertyMetadata.Values.RemoveAll([AvailableMetaDataTypes](const FInstancedStruct& RemovalCandidate)
		{
			return !AvailableMetaDataTypes.Contains(RemovalCandidate.GetScriptStruct());
		});
		
		for (UScriptStruct* AvailableType : AvailableMetaDataTypes)
		{
			GetDataMutable<UPropertyBagHierarchyProperty>()->FindOrAddPropertyMetaData(AvailableType);
		}
		
		// In the end we sort to the same order as we the schema has defined
		GetDataMutable<UPropertyBagHierarchyProperty>()->PropertyMetadata.Values.Sort([AvailableMetaDataTypes](const FInstancedStruct& StructA, const FInstancedStruct& StructB)
		{
			return AvailableMetaDataTypes.IndexOfByKey(StructA.GetScriptStruct()) < AvailableMetaDataTypes.IndexOfByKey(StructB.GetScriptStruct()); 
		});
	}
}

bool FPropertyBagHierarchyPropertyViewModel::DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const
{
	// If the property desc doesn't exist anymore, we delete it	
	return GetPropertyDesc() ? true : false;
}

void FPropertyBagHierarchyPropertyViewModel::SyncViewModelsToDataInternal()
{
	UpdateMetaData();
}

const FPropertyBagPropertyDesc* FPropertyBagHierarchyPropertyViewModel::GetPropertyDesc() const
{
	return Cast<UPropertyBagHierarchyViewModel>(HierarchyViewModel)->GetPropertyDescForProperty(GetData<UPropertyBagHierarchyProperty>()->GetPropertyId());
}

void FPropertyBagHierarchyPropertyViewModel::PostEditChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (PropertyChangedEvent.Property)
	{
		if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(PropertyChangedEvent.Property->Owner.ToUObject()))
		{
			FInstancedStruct* FoundStruct = GetDataMutable<UPropertyBagHierarchyProperty>()->PropertyMetadata.Values.FindByPredicate([ScriptStruct](const FInstancedStruct& Candidate)
			{
				return Candidate.GetScriptStruct() == ScriptStruct;
			});
			
			if (FoundStruct && FoundStruct->GetScriptStruct()->IsChildOf<FPropertyBagPropertyMetadata_Base>())
			{
				FoundStruct->GetMutablePtr<FPropertyBagPropertyMetadata_Base>()->PostEditChange(PropertyChangedEvent, SharedThis(this));
			}
		}
	}
	
	Cast<UPropertyBagHierarchyViewModel>(HierarchyViewModel)->RebuildDetails();
}

FString FPropertyBagHierarchyPropertyViewModel::ToString() const
{
	const FPropertyBagPropertyDesc* Desc = GetPropertyDesc();
	return Desc ? Desc->Name.ToString() : TEXT("Unidentified");
}

void UPropertyBagHierarchyViewModel::Initialize(UObject* InOwningObject, TSharedRef<FPropertyPath> InPropertyPath)
{
	if (!InOwningObject)
	{
		return;
	}

	OwningObject = InOwningObject;
	PropertyPathToPropertyBag = InPropertyPath;

	// We generate our own property handle via a PropertyRowGenerator. This gives us a stable handle
	// that survives details panel refreshes (which would invalidate the original handle).
	FPropertyRowGeneratorArgs Args;
	PropertyRowGenerator = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(Args);
	PropertyRowGenerator->SetObjects({InOwningObject});
	RegeneratePropertyBagHandle();

	// Extract schema and hierarchy root from our generated handle
	if (GeneratedPropertyBagHandle.IsValid())
	{
		PropertyBagSchema = UE::StructUtils::ExtractPropertyBagSchemaCDO(GeneratedPropertyBagHandle.ToSharedRef());
		HierarchyRoot = UE::StructUtils::ExtractHierarchyRoot(GeneratedPropertyBagHandle.ToSharedRef());
	}

	// We add the properties that don't currently exist in the hierarchy to the root.
	AddMissingPropertiesToRoot(false);

	// Initialize the hash to avoid a false-positive change detection on the first tick
	LastHash = UE::StructUtils::CalcPropertyDescArrayHash(GetPropertyDescs());

	Super::Initialize();
}

void UPropertyBagHierarchyViewModel::BeginDestroy()
{
	Finalize();
	
	Super::BeginDestroy();
}

void UPropertyBagHierarchyViewModel::RegeneratePropertyBagHandle()
{
	if (!PropertyPathToPropertyBag.IsValid() || !PropertyRowGenerator.IsValid())
	{
		return;
	}
	
	if (TSharedPtr<IDetailTreeNode> TreeNode = FindNode(PropertyRowGenerator->GetRootTreeNodes(), PropertyPathToPropertyBag.ToSharedRef()))
	{
		GeneratedPropertyBagHandle = TreeNode->CreatePropertyHandle();
	}
}

const FPropertyBagPropertyDesc* UPropertyBagHierarchyViewModel::GetPropertyDescForProperty(FGuid PropertyId) const
{
	if (const FInstancedPropertyBag* PropertyBag = GetPropertyBag())
	{
		if (const UPropertyBag* PropertyBagStruct = PropertyBag->GetPropertyBagStruct())
		{
			return PropertyBagStruct->FindPropertyDescByID(PropertyId);
		}
	}
	
	return nullptr;
}

TConstArrayView<FPropertyBagPropertyDesc> UPropertyBagHierarchyViewModel::GetPropertyDescs() const
{
	if (const FInstancedPropertyBag* PropertyBag = GetPropertyBag())
	{
		if (const UPropertyBag* PropertyBagStruct = PropertyBag->GetPropertyBagStruct())
		{
			return PropertyBagStruct->GetPropertyDescs();
		}
	}
	
	return {};
}

void UPropertyBagHierarchyViewModel::SetPropertyUtilities(TSharedRef<IPropertyUtilities> InPropertyUtilities)
{
	PropertyUtilities = InPropertyUtilities;
}

void UPropertyBagHierarchyViewModel::RebuildDetails() const
{
	// We can't request a rebuild from the handle itself since it's not the same handle as in the details panel.
	// However, the property utilities remain stable, so we can force a refresh. This would have also invalidated the original handle.
	if (PropertyUtilities.IsValid())
	{
		PropertyUtilities.Pin()->RequestForceRefresh();
	}
}

void UPropertyBagHierarchyViewModel::Tick(float DeltaTime)
{
	// If our handle becomes invalid, we regenerate it. This can happen when adding new properties for example
	if (!GeneratedPropertyBagHandle.IsValid() || !GeneratedPropertyBagHandle->IsValidHandle())
	{
		RegeneratePropertyBagHandle();
	}

	// Detect structural changes to the property bag (new/removed properties) and refresh accordingly
	uint64 CurrentHash = UE::StructUtils::CalcPropertyDescArrayHash(GetPropertyDescs());
	if (LastHash != CurrentHash)
	{
		AddMissingPropertiesToRoot(false);
		RequestFullRefreshNextFrame();
		LastHash = CurrentHash;
	}
}

bool UPropertyBagHierarchyViewModel::IsTickable() const
{
	// Stop ticking the instant the last FPropertyBagHierarchyViewModelOwner is released:
	// the owner destructor calls Finalize() (flipping IsFinalized()) and MarkAsGarbage()
	// (setting RF_MirroredGarbage / IsUnreachable becomes true on the next GC).
	return !HasAnyFlags(RF_ClassDefaultObject | RF_MirroredGarbage)
		&& !IsUnreachable()
		&& IsInitialized() && !IsFinalized();
}

TStatId UPropertyBagHierarchyViewModel::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(PropertyBagHierarchyViewModel, STATGROUP_Tickables);
}

UHierarchyRoot* UPropertyBagHierarchyViewModel::GetHierarchyRoot() const
{
	return HierarchyRoot.Get();
}

void UPropertyBagHierarchyViewModel::OnHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload)
{
	Super::OnHierarchyChanged(Payload);
	
	if(Payload.IsValid())
	{
		if(Payload.GetScriptStruct() == FHierarchyChangedPayload_ElementsDeleted::StaticStruct())
		{
			TArray<TSharedPtr<FHierarchyElementViewModel>> DeletedElementViewModel = Payload.GetPtr<FHierarchyChangedPayload_ElementsDeleted>()->DeletedElementViewModels;
			
			bool bContainsHierarchyElement = DeletedElementViewModel.ContainsByPredicate([](TSharedPtr<FHierarchyElementViewModel> Candidate)
			{
				return Candidate->GetData()->IsA<UHierarchyCategory>() || Candidate->GetData()->IsA<UHierarchyItem>();
			});
			
			// By deleting a category we might have deleted contained properties. Readd them to the hierarchy.
			// Deleting properties isn't possible to do directly, but it might happen (e.g.: changing property type that warrants a new hierarchy type)
			if (bContainsHierarchyElement)
			{
				AddMissingPropertiesToRoot(true);
			}
		}
	}
	
	RebuildDetails();
}

TSubclassOf<UHierarchyCategory> UPropertyBagHierarchyViewModel::GetCategoryDataClass() const
{
	if (PropertyBagSchema.IsValid())
	{
		return PropertyBagSchema->GetHierarchyCategoryType();
	}
	
	return nullptr;
}

TSubclassOf<UHierarchySection> UPropertyBagHierarchyViewModel::GetSectionDataClass() const
{
	if (PropertyBagSchema.IsValid())
	{
		return PropertyBagSchema->GetHierarchySectionType();
	}
	
	return nullptr;
}

TSharedPtr<FHierarchyElementViewModel> UPropertyBagHierarchyViewModel::CreateCustomViewModelForElement(UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> Parent)
{	
	if (UPropertyBagHierarchyProperty* Property = Cast<UPropertyBagHierarchyProperty>(Element))
	{
		return MakeShared<FPropertyBagHierarchyPropertyViewModel>(Property, Parent.ToSharedRef(), this);
	}
	if(UHierarchyRoot* Root = Cast<UHierarchyRoot>(Element))
	{
		// If the root is the hierarchy root, we know it's for the hierarchy. If not, it's the transient source root
		bool bIsForHierarchy = GetHierarchyRoot() == Element;
		return MakeShared<FPropertyBagHierarchyScriptRootViewModel>(Root, this, bIsForHierarchy);
	}
	
	return nullptr;
}

void UPropertyBagHierarchyViewModel::PostUndo(bool bSuccess)
{
	RebuildDetails();
	
	Super::PostUndo(bSuccess);
}

void UPropertyBagHierarchyViewModel::PostRedo(bool bSuccess)
{
	RebuildDetails();
	
	Super::PostRedo(bSuccess);
}

const FInstancedPropertyBag* UPropertyBagHierarchyViewModel::GetPropertyBag() const
{
	if (PropertyPathToPropertyBag.IsValid() && OwningObject.IsValid())
	{
		return PropertyPathToPropertyBag->GetLeafMostProperty().Property->ContainerPtrToValuePtr<FInstancedPropertyBag>(OwningObject.Get());
	}
	
	if (GeneratedPropertyBagHandle.IsValid() && GeneratedPropertyBagHandle->IsValidHandle())
	{
		void* DataPtr = nullptr;
		if (GeneratedPropertyBagHandle->GetValueData(DataPtr) == FPropertyAccess::Result::Success)
		{
			return (const FInstancedPropertyBag*) DataPtr;
		}
	}
	
	return nullptr; 
}

void UPropertyBagHierarchyViewModel::AddMissingPropertiesToRoot(bool bSendNotification)
{
	if (!HierarchyRoot.IsValid() || !PropertyBagSchema.IsValid())
	{
		return;
	}

	const FInstancedPropertyBag* InstancedPropertyBag = GetPropertyBag();
	const UPropertyBag* PropertyBag = InstancedPropertyBag ? InstancedPropertyBag->GetPropertyBagStruct() : nullptr;

	// If the property bag is null or empty, there is nothing to do
	if (!PropertyBag || PropertyBag->GetPropertyDescs().IsEmpty())
	{
		return;
	}

	TArray<UPropertyBagHierarchyProperty*> NewProperties;

	TArray<const UPropertyBagHierarchyProperty*> AllContainedProperties;
	HierarchyRoot->GetChildrenOfType(AllContainedProperties, true);
	
	for (const FPropertyBagPropertyDesc& Desc : PropertyBag->GetPropertyDescs())
	{
		if (!AllContainedProperties.ContainsByPredicate([Desc](const UPropertyBagHierarchyProperty* Candidate)
		{
			return Candidate->GetPropertyId() == Desc.ID;
		}))
		{
			UPropertyBagHierarchyProperty* NewHierarchyProperty = NewObject<UPropertyBagHierarchyProperty>(HierarchyRoot.Get());
			NewHierarchyProperty->Initialize(Desc);
			HierarchyRoot->GetChildrenMutable().Add(NewHierarchyProperty);
			NewProperties.Add(NewHierarchyProperty);
		}
	}

	if (NewProperties.Num() > 0)
	{
		if (HierarchyRootViewModel.IsValid())
		{
			HierarchyRootViewModel->SyncViewModelsToData();
		}
		
		if (bSendNotification)
		{
			FNotificationInfo Info(FText::FormatOrdered(LOCTEXT("AddedMissingPropertiesToRoot", "Added missing {0}|plural(one=property, other=properties) to the hierarchy root."), NewProperties.Num()));
			FSlateNotificationManager::Get().AddNotification(Info);
		}
	}
}

bool UPropertyBagHierarchyViewModel::FindPropertyHandleRecursive(const TSharedPtr<IPropertyHandle>& PropertyHandle, TSharedRef<FPropertyPath> PropertyPath)
{
	if (PropertyHandle && PropertyHandle->IsValidHandle())
	{
		uint32 ChildrenCount = 0;
		PropertyHandle->GetNumChildren(ChildrenCount);
		for (uint32 Index = 0; Index < ChildrenCount; ++Index)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(Index);
			if (FindPropertyHandleRecursive(ChildHandle, PropertyPath))
			{
				return true;
			}
		}

		if (PropertyHandle->GetProperty())
		{
			if (FPropertyPath::AreEqual(PropertyHandle->CreateFPropertyPath().ToSharedRef(), PropertyPath))
			{
				return true;
			}
		}
	}

	return false;
}

TSharedPtr<IDetailTreeNode> UPropertyBagHierarchyViewModel::FindTreeNodeRecursive(const TSharedRef<IDetailTreeNode>& RootNode, TSharedRef<FPropertyPath> PropertyPath)
{
	TArray<TSharedRef<IDetailTreeNode>> Children;
	RootNode->GetChildren(Children);
	for (TSharedRef<IDetailTreeNode>& Child : Children)
	{
		TSharedPtr<IDetailTreeNode> FoundNode = FindTreeNodeRecursive(Child, PropertyPath);
		if (FoundNode.IsValid())
		{
			return FoundNode;
		}
	}

	TSharedPtr<IPropertyHandle> Handle = RootNode->CreatePropertyHandle();
	if (FindPropertyHandleRecursive(Handle, PropertyPath))
	{
		return RootNode;
	}

	return nullptr;
}

TSharedPtr<IDetailTreeNode> UPropertyBagHierarchyViewModel::FindNode(const TArray<TSharedRef<IDetailTreeNode>>& RootNodes, TSharedRef<FPropertyPath> PropertyPath)
{
	for (const TSharedRef<IDetailTreeNode>& CategoryNode : RootNodes)
	{
		TSharedPtr<IDetailTreeNode> FoundNode = FindTreeNodeRecursive(CategoryNode, PropertyPath);
		if (FoundNode.IsValid())
		{
			return FoundNode;
		}
	}

	return nullptr;
}

void FPropertyBagHierarchyScriptRootViewModel::PostEditHierarchyStructure()
{
	FHierarchyRootViewModel::PostEditHierarchyStructure();
	if (UPropertyBagHierarchyRoot* PropBagRootViewModel = Cast<UPropertyBagHierarchyRoot>(Element))
	{
		PropBagRootViewModel->OnHierarchyModified.Broadcast();
	}
}

FPropertyBagHierarchyViewModelPropertyEditorPolicy::FPropertyBagHierarchyViewModelPropertyEditorPolicy()
{
	PropertyEditorPolicy::RegisterArchetypePolicy(this);
}

FPropertyBagHierarchyViewModelPropertyEditorPolicy::~FPropertyBagHierarchyViewModelPropertyEditorPolicy()
{
	PropertyEditorPolicy::UnregisterArchetypePolicy(this);
}

UObject* FPropertyBagHierarchyViewModelPropertyEditorPolicy::GetArchetypeForObject(const UObject* Object) const
{
	if (const UPropertyBagHierarchyProperty* PropertyBagProperty = Cast<UPropertyBagHierarchyProperty>(Object))
	{
		return PropertyBagProperty->GetArchetype();
	}
	return {};
}

#undef LOCTEXT_NAMESPACE
