// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPropertyBagHierarchyEditor.h"
#include "PropertyBagDetails.h"
#include "StructUtilsEditorModule.h"
#include "HierarchyEditor/PropertyBagHierarchyViewModel.h"
#include "PropertyPath.h"
#include "SlateOptMacros.h"
#include "StructUtilsEditorLog.h"
#include "StructUtilsEditorUtilsPrivate.h"
#include "StructUtilsMetadata.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/SDataHierarchyEditor.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Editor.h"
#include "IPropertyUtilities.h"

#define LOCTEXT_NAMESPACE "PropertyBagHierarchyEditor"

void SPropertyBagHierarchyEditor::Construct(const FArguments& InArgs, TSharedRef<FPropertyBagHierarchyViewModelOwner> InHierarchyViewModelOwner)
{
	OnCloseRequested = InArgs._OnCloseRequested;

	HierarchyViewModelOwner = InHierarchyViewModelOwner;
	UPropertyBagHierarchyViewModel* HierarchyViewModel = HierarchyViewModelOwner->Get();

	TSharedPtr<IPropertyHandle> PropertyBagHandle = HierarchyViewModel->GetPropertyBagHandle();
	if (!PropertyBagHandle.IsValid())
	{
		UE_LOGF(LogStructUtilsEditor, Warning, "SPropertyBagHierarchyEditor::Construct — HierarchyViewModel has no valid property bag handle.");
		return;
	}

	PropertyBagSchemaCDO = UE::StructUtils::ExtractPropertyBagSchemaCDO(PropertyBagHandle.ToSharedRef());
	HierarchyRoot = HierarchyViewModel->GetHierarchyRoot();

	if (!PropertyBagSchemaCDO.IsValid())
	{
		UE_LOGF(LogStructUtilsEditor, Warning, "SPropertyBagHierarchyEditor::Construct — Failed to extract PropertyBagSchema CDO.");
		return;
	}

	if (!HierarchyRoot.IsValid())
	{
		UE_LOGF(LogStructUtilsEditor, Warning, "SPropertyBagHierarchyEditor::Construct — HierarchyViewModel has no valid hierarchy root.");
		return;
	}

	TArray<UObject*> OutOuterObjects;
	PropertyBagHandle->GetOuterObjects(OutOuterObjects);
	Algo::Transform(OutOuterObjects, OuterObjects, [](UObject* Object)
	{
		return TWeakObjectPtr<UObject>(Object);
	});

	if (GEditor)
	{
		if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			AssetEditorSubsystem->OnAssetClosedInEditor().AddSP(this, &SPropertyBagHierarchyEditor::OnAssetEditorClosed);
		}
	}

	ChildSlot
	[
		SNew(SDataHierarchyEditor, HierarchyViewModel)
		.OnGenerateRowContentWidget(this, &SPropertyBagHierarchyEditor::OnGenerateRowContentWidget)
	];
}

TSharedPtr<IPropertyHandle> SPropertyBagHierarchyEditor::GetPropertyBagHandle() const
{
	if (HierarchyViewModelOwner.IsValid())
	{
		return HierarchyViewModelOwner->Get()->GetPropertyBagHandle();
	}

	return nullptr;
}

SPropertyBagHierarchyEditor::~SPropertyBagHierarchyEditor()
{
	if (GEditor)
	{
		if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			AssetEditorSubsystem->OnAssetClosedInEditor().RemoveAll(this);
		}
	}

	HierarchyViewModelOwner.Reset();
}

void SPropertyBagHierarchyEditor::NavigateToProperty(const FPropertyBagPropertyDesc& PropertyDesc, bool bAddIfNotFound)
{
	if (!HierarchyViewModelOwner.IsValid())
	{
		return;
	}

	UPropertyBagHierarchyViewModel* HierarchyViewModel = HierarchyViewModelOwner->Get();

	FHierarchyElementIdentity Identity = UPropertyBagHierarchyProperty::ConstructIdentity(PropertyDesc);
	TSharedPtr<FHierarchyElementViewModel> FoundViewModel = HierarchyViewModel->GetHierarchyRootViewModel()->FindViewModelForChild(Identity, true);

	if (!FoundViewModel.IsValid())
	{
		if (bAddIfNotFound)
		{
			UHierarchyElement* Root = HierarchyViewModel->GetHierarchyRootViewModel()->GetDataMutable();

			UPropertyBagHierarchyProperty* NewHierarchyProperty = NewObject<UPropertyBagHierarchyProperty>(Root);
			NewHierarchyProperty->Initialize(PropertyDesc);
			Root->GetChildrenMutable().Add(NewHierarchyProperty);
			HierarchyViewModel->GetHierarchyRootViewModel()->SyncViewModelsToData();
			FoundViewModel = HierarchyViewModel->GetHierarchyRootViewModel()->FindViewModelForChild(Identity, false);

			HierarchyViewModel->SetActiveHierarchySection(nullptr);

			HierarchyViewModel->RebuildDetails();

			FNotificationInfo Info(FText::FormatOrdered(LOCTEXT("MissingHierarchyPropertyAdded", "Property {0} was added to the hierarchy so it can be edited."),
				FText::FromName(PropertyDesc.Name)));

			FSlateNotificationManager::Get().AddNotification(Info);
		}
	}

	// We delay navigation by a short duration; the window containing the hierarchy editor might have been added this same frame
	// in which case the navigation would fail. 0.25 seems like a good magic number
	RegisterActiveTimer(0.25f, FWidgetActiveTimerDelegate::CreateLambda([this, Identity](double InCurrentTime, float InDeltaTime)
	{
		if (HierarchyViewModelOwner.IsValid())
		{
			UPropertyBagHierarchyViewModel* VM = HierarchyViewModelOwner->Get();
			VM->NavigateToElementInHierarchy(VM->GetHierarchyRootViewModel()->FindViewModelForChild(Identity, true).ToSharedRef());
		}

		return EActiveTimerReturnType::Stop;
	}));
}

TSharedRef<SWidget> SPropertyBagHierarchyEditor::OnGenerateRowContentWidget(TSharedRef<FHierarchyElementViewModel> ElementViewModel) const
{
	if (ElementViewModel->GetDataMutable()->IsA<UPropertyBagHierarchyProperty>())
	{
		TSharedRef<FPropertyBagHierarchyPropertyViewModel> HierarchyPropertyViewModel = StaticCastSharedRef<FPropertyBagHierarchyPropertyViewModel>(ElementViewModel);
				
		const FPropertyBagPropertyDesc* Desc = HierarchyPropertyViewModel->GetPropertyDesc();
		FEdGraphPinType PinType = Desc ? UE::StructUtils::GetPropertyDescAsPin(*Desc) : FEdGraphPinType();
		
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			[
				SNew(SImage)
				.Image(FBlueprintEditorUtils::GetIconFromPin(PinType))
				.ColorAndOpacity(PropertyBagSchemaCDO->GetPinTypeColor(PinType))
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SHierarchyElement, HierarchyPropertyViewModel)
			];
	}
	
	if(ElementViewModel->GetDataMutable()->IsA<UPropertyBagHierarchyCategory>())
	{
		TSharedRef<FHierarchyCategoryViewModel> HierarchyCategoryViewModel = StaticCastSharedRef<FHierarchyCategoryViewModel>(ElementViewModel);
		return SNew(SHierarchyElement, HierarchyCategoryViewModel)
			.Style(&FDataHierarchyEditorStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("HierarchyEditor.Category"));
	}
			
	return SNew(STextBlock).Text(ElementViewModel->ToStringAsText());
}

void SPropertyBagHierarchyEditor::OnAssetEditorClosed(UObject* Object, IAssetEditorInstance* AssetEditorInstance) const
{
	if (OuterObjects.Contains(Object))
	{
		OnCloseRequested.ExecuteIfBound();
	}
}

#undef LOCTEXT_NAMESPACE
