// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataHierarchyViewModelBase.h"
#include "Misc/NotNull.h"
#include "PropertyBagDetails.h"
#include "EdGraph/EdGraphSchema.h"
#include "HierarchyEditor/PropertyBagHierarchyViewModel.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/SubclassOf.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API STRUCTUTILSEDITOR_API

class UPropertyBagSchema;
class UPropertyBagHierarchyViewModel;
struct FPropertyBagHierarchyViewModelOwner;

/** A wrapper around the actual hierarchy editor. Receives and holds a shared reference to the view model. */
class SPropertyBagHierarchyEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPropertyBagHierarchyEditor)
		{}
		SLATE_EVENT(FSimpleDelegate, OnCloseRequested)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedRef<FPropertyBagHierarchyViewModelOwner> InHierarchyViewModelOwner);
	UE_API virtual ~SPropertyBagHierarchyEditor() override;

	/** Returns the property handle that we generated ourselves via property row generator. */
	UE_API TSharedPtr<IPropertyHandle> GetPropertyBagHandle() const;
	/** Returns the hierarchy root the editor is currently bound to. Used by the summoner to decide whether
	 *  an already-open window can be reused as-is for a new request. */
	UHierarchyRoot* GetHierarchyRoot() const { return HierarchyRoot.Get(); }
	UE_API void NavigateToProperty(const FPropertyBagPropertyDesc& PropertyDesc, bool bAddIfNotFound = false);

private:
	TSharedRef<SWidget> OnGenerateRowContentWidget(TSharedRef<FHierarchyElementViewModel> ElementViewModel) const;
	void OnAssetEditorClosed(UObject* Object, IAssetEditorInstance* AssetEditorInstance) const;

private:
	TWeakObjectPtr<const UPropertyBagSchema> PropertyBagSchemaCDO;
	/** Shared ownership — keeps the view model alive as long as this widget exists, even if the details panel is destroyed.
	 *  When the last owner is released, the view model is finalized and marked as garbage. */
	TSharedPtr<FPropertyBagHierarchyViewModelOwner> HierarchyViewModelOwner;

	TWeakObjectPtr<UHierarchyRoot> HierarchyRoot;

	TArray<TWeakObjectPtr<UObject>> OuterObjects;

	FSimpleDelegate OnCloseRequested;
};

#undef UE_API
