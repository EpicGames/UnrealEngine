// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedStructDetails.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "UObject/Package.h"
#include "Widgets/Images/SImage.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "StructUtils/UserDefinedStruct.h"
#include "StructUtils/InstancedStruct.h"
#include "IStructureDataProvider.h"
#include "Misc/ConfigCacheIni.h"
#include "StructUtilsDelegates.h"
#include "StructUtilsMetadata.h"
#include "SInstancedStructPicker.h"
#include "StructUtils/InstancedStructContainer.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "StructUtilsEditor"

////////////////////////////////////

void FInstancedStructProvider::Reset()
{
	StructProperty = nullptr;
}

bool FInstancedStructProvider::IsValid() const
{
	bool bHasValidData = false;
	EnumerateInstances([&bHasValidData](const UScriptStruct* ScriptStruct, uint8* Memory, UPackage* Package)
	{
		if (ScriptStruct && Memory)
		{
			bHasValidData = true;
			return false; // Stop
		}
		return true; // Continue
	});

	return bHasValidData;
}

const UStruct* FInstancedStructProvider::GetBaseStructure() const
{
	// Taken from UClass::FindCommonBase
	auto FindCommonBaseStruct = [](const UScriptStruct* StructA, const UScriptStruct* StructB)
	{
		const UScriptStruct* CommonBaseStruct = StructA;
		while (CommonBaseStruct && StructB && !StructB->IsChildOf(CommonBaseStruct))
		{
			CommonBaseStruct = Cast<UScriptStruct>(CommonBaseStruct->GetSuperStruct());
		}
		return CommonBaseStruct;
	};

	const UScriptStruct* CommonStruct = nullptr;
	EnumerateInstances([&CommonStruct, &FindCommonBaseStruct](const UScriptStruct* ScriptStruct, uint8* Memory, UPackage* Package)
	{
		if (ScriptStruct)
		{
			CommonStruct = FindCommonBaseStruct(ScriptStruct, CommonStruct);
		}
		return true; // Continue
	});

	return CommonStruct;
}

void FInstancedStructProvider::GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* ExpectedBaseStructure) const
{
	// The returned instances need to be compatible with base structure.
	// This function returns empty instances in case they are not compatible, with the idea that we have as many instances as we have outer objects.
	EnumerateInstances([&OutInstances, ExpectedBaseStructure](const UScriptStruct* ScriptStruct, uint8* Memory, UPackage* Package)
	{
		TSharedPtr<FStructOnScope> Result;
		
		if (ExpectedBaseStructure && ScriptStruct && ScriptStruct->IsChildOf(ExpectedBaseStructure))
		{
			Result = MakeShared<FStructOnScope>(ScriptStruct, Memory);
			Result->SetPackage(Package);
		}

		OutInstances.Add(Result);

		return true; // Continue
	});
}

bool FInstancedStructProvider::IsPropertyIndirection() const
{
	return true;
}

uint8* FInstancedStructProvider::GetValueBaseAddress(uint8* ParentValueAddress, const UStruct* ExpectedBaseStructure) const
{
	if (!ParentValueAddress)
	{
		return nullptr;
	}

	FInstancedStruct& InstancedStruct = *reinterpret_cast<FInstancedStruct*>(ParentValueAddress);
	if (ExpectedBaseStructure && InstancedStruct.GetScriptStruct() && InstancedStruct.GetScriptStruct()->IsChildOf(ExpectedBaseStructure))
	{
		return InstancedStruct.GetMutableMemory();
	}
	
	return nullptr;
}

void FInstancedStructProvider::EnumerateInstances(TFunctionRef<bool(const UScriptStruct* ScriptStruct, uint8* Memory, UPackage* Package)> InFunc) const
{
	if (!StructProperty.IsValid() || !StructProperty->IsValidHandle())
	{
		return;
	}
	
	TArray<UPackage*> Packages;
	StructProperty->GetOuterPackages(Packages);

	StructProperty->EnumerateRawData([&InFunc, &Packages](void* RawData, const int32 DataIndex, const int32 /*NumDatas*/)
	{
		const UScriptStruct* ScriptStruct = nullptr;
		uint8* Memory = nullptr;
		UPackage* Package = nullptr;
		if (FInstancedStruct* InstancedStruct = static_cast<FInstancedStruct*>(RawData))
		{
			ScriptStruct = InstancedStruct->GetScriptStruct();
			Memory = InstancedStruct->GetMutableMemory();
			if (ensureMsgf(Packages.IsValidIndex(DataIndex), TEXT("Expecting packges and raw data to match.")))
			{
				Package = Packages[DataIndex];
			}
		}

		return InFunc(ScriptStruct, Memory, Package);
	});
}

////////////////////////////////////

FInstancedStructDataDetails::FInstancedStructDataDetails(TSharedPtr<IPropertyHandle> InStructProperty)
{
#if DO_CHECK
	FStructProperty* StructProp = CastFieldChecked<FStructProperty>(InStructProperty->GetProperty());
	check(StructProp);
	check(StructProp->Struct == FInstancedStruct::StaticStruct());
#endif

	StructProperty = InStructProperty;
}

FInstancedStructDataDetails::~FInstancedStructDataDetails()
{
	UE::StructUtils::Delegates::OnUserDefinedStructReinstanced.Remove(UserDefinedStructReinstancedHandle);
}

TArray<FString> FInstancedStructDataDetails::GetPropertyCategories(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	static const FName CategoryName = FName(TEXT("Category"));
	static const FName EnableCategoriesName = FName(TEXT("EnableCategories"));

	// The property needs the "EnableCategories" metadata in order to be added under a group. Grouping is opt-in.
	if (!PropertyHandle->HasMetaData(EnableCategoriesName))
	{
		return { };
	}

	const FString& PropertyCategory = PropertyHandle->GetMetaData(CategoryName).TrimStartAndEnd();
	if (PropertyCategory.IsEmpty())
	{
		return { };
	}

	constexpr bool bCullEmpty = true;
	TArray<FString> CategoriesToAdd;
	PropertyCategory.ParseIntoArray(CategoriesToAdd, TEXT("|"), bCullEmpty);

	// Clean up categories
	for (int32 Index = 0; Index < CategoriesToAdd.Num(); ++Index)
	{
		FString& CategoryToAdd = CategoriesToAdd[Index];
		CategoryToAdd.TrimStartAndEndInline();

		// Cover the edge case where there's a category like "Foo|", which is invalid
		CategoryToAdd.TrimCharInline(TEXT('|'), nullptr);
	}

	return CategoriesToAdd;
}

void FInstancedStructDataDetails::OnUserDefinedStructReinstancedHandle(const UUserDefinedStruct& Struct)
{
	OnStructLayoutChanges();
}

void FInstancedStructDataDetails::SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren)
{
	OnRegenerateChildren = InOnRegenerateChildren;
}

TArray<TWeakObjectPtr<const UStruct>> FInstancedStructDataDetails::GetInstanceTypes() const
{
	TArray<TWeakObjectPtr<const UStruct>> Result;
	
	if (!StructProperty.IsValid() || !StructProperty->IsValidHandle())
	{
		return Result;
	}
	
	StructProperty->EnumerateConstRawData([&Result](const void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
	{
		TWeakObjectPtr<const UStruct>& Type = Result.AddDefaulted_GetRef();
		if (const FInstancedStruct* InstancedStruct = static_cast<const FInstancedStruct*>(RawData))
		{
			Result.Add(InstancedStruct->GetScriptStruct());
		}
		else
		{
			Result.Add(nullptr);
		}
		return true;
	});

	return Result;
}

void FInstancedStructDataDetails::GetPropertyGroups(
	const TArray<TSharedPtr<IPropertyHandle>>& InProperties,
	IDetailChildrenBuilder& InChildBuilder,
	TMap<TSharedPtr<IPropertyHandle>, IDetailGroup*>& OutPropertyToGroup) const
{
	// Temporarily store a mapping of category -> group while groups are being built
	TMap<FString, IDetailGroup*> CategoryToGroup;
	
	for (const TSharedPtr<IPropertyHandle>& PropertyHandle : InProperties)
	{
		TArray<FString> CategoriesToAdd = GetPropertyCategories(PropertyHandle);
		if (CategoriesToAdd.IsEmpty())
		{
			continue;
		}

		// Tracks the category name as it is being built up (eg, Foo -> Foo|Bar -> Foo|Bar|Baz)
		FString CompleteCategory;

		// For this property, add all of the groups needed for its category (eg, Foo, Foo|Bar, and Foo|Bar|Baz)
		IDetailGroup* CurrentGroup = nullptr;
		for (int32 Index = 0; Index < CategoriesToAdd.Num(); ++Index)
		{
			FString& CategoryToAdd = CategoriesToAdd[Index];
			CompleteCategory = CompleteCategory.IsEmpty()
				? CategoryToAdd
				: FString::Join(TArray({CompleteCategory, CategoryToAdd}), TEXT("|"));

			// Create the category's group if it has not yet been created
			if (!CategoryToGroup.Contains(CompleteCategory))
			{
				if (CurrentGroup)
				{
					// Add the group to the previous group if this is a nested category (eg, if this is the Foo|Bar group, add to the Foo group)
					CurrentGroup = &CurrentGroup->AddGroup(FName(CompleteCategory), FText::FromString(CategoryToAdd));
				}
				else
				{
					// Otherwise, add the group as a normal group via the builder
					CurrentGroup = &InChildBuilder.AddGroup(FName(CompleteCategory), FText::FromString(CategoryToAdd));
				}

				OnGroupRowAdded(*CurrentGroup, Index, CategoryToAdd);
				CategoryToGroup.Add(CompleteCategory, CurrentGroup);
			}
			else
			{
				CurrentGroup = CategoryToGroup[CompleteCategory];
			}
		}

		check(CurrentGroup);
		OutPropertyToGroup.Add(PropertyHandle, CurrentGroup);
	}
}

void FInstancedStructDataDetails::OnStructLayoutChanges()
{
	bCanHandleStructValuePostChange = false;
	bStructChangeDetected = true;
}

FInstancedStructDataWithHeaderDetails::FInstancedStructDataWithHeaderDetails(TSharedPtr<IPropertyHandle> InStructProperty)
	: FInstancedStructDataDetails(InStructProperty)
{
}

void FInstancedStructDataWithHeaderDetails::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	FInstancedStructDataDetails::GenerateHeaderRowContent(NodeRow);

	// Use EnumerateConstRawData instead of GetValueData to support multi-object editing.
	// GetValueData requires byte-identical data across all objects, but we only need the struct type for the display name.
	const UScriptStruct* CommonStructType = nullptr;
	bool bAllSameType = true;
	StructProperty->EnumerateConstRawData([&CommonStructType, &bAllSameType](const void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
	{
		if (const FInstancedStruct* StructData = static_cast<const FInstancedStruct*>(RawData))
		{
			if (const UScriptStruct* StructType = StructData->GetScriptStruct())
			{
				if (CommonStructType == nullptr)
				{
					CommonStructType = StructType;
				}
				else if (CommonStructType != StructType)
				{
					bAllSameType = false;
					return false;
				}
			}
			else
			{
				return false;
			}
		}
			
		return true;
	});

	if (CommonStructType && bAllSameType)
	{
		NodeRow.WholeRowWidget.Widget = SNew(SBox)
		.Padding(4.f, 0.f, 0.f, 0.f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(CommonStructType->GetDisplayNameText())
			.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
			.Font(FAppStyle::Get().GetFontStyle("DetailsView.CategoryFontStyle"))
		];
	}
}

FInstancedStructArrayBuilder::FInstancedStructArrayBuilder(TSharedRef<IPropertyHandle> InArrayHandle)
	: FDetailArrayBuilder(InArrayHandle, false, false, false)
{
}

void FInstancedStructArrayBuilder::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	FDetailArrayBuilder::GenerateHeaderRowContent(NodeRow);
}

void FInstancedStructArrayBuilder::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	FDetailArrayBuilder::GenerateChildContent(ChildrenBuilder);
}

void FInstancedStructDataDetails::OnStructHandlePostChange()
{
	if (bCanHandleStructValuePostChange)
	{
		TArray<TWeakObjectPtr<const UStruct>> InstanceTypes = GetInstanceTypes();
		if (InstanceTypes != CachedInstanceTypes)
		{
			bStructChangeDetected = false;
			OnRegenerateChildren.ExecuteIfBound();
		}
	}
}

void FInstancedStructDataDetails::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	StructProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FInstancedStructDataDetails::OnStructHandlePostChange));
	if (!UserDefinedStructReinstancedHandle.IsValid())
	{
		UserDefinedStructReinstancedHandle = UE::StructUtils::Delegates::OnUserDefinedStructReinstanced.AddSP(this, &FInstancedStructDataDetails::OnUserDefinedStructReinstancedHandle);
	}
}

void FInstancedStructDataDetails::GenerateChildContent(IDetailChildrenBuilder& ChildBuilder)
{
	// Add the rows for the struct
	TSharedRef<FInstancedStructProvider> NewStructProvider = MakeShared<FInstancedStructProvider>(StructProperty);

	bool bCustomizedProperty = false;

	const UStruct* BaseStruct = NewStructProvider->GetBaseStructure();
	if (BaseStruct)
	{
		static const FName EditModuleName("PropertyEditor");
		if (const FPropertyEditorModule* EditModule = FModuleManager::GetModulePtr<FPropertyEditorModule>(EditModuleName))
		{
			if (EditModule->IsCustomizedStruct(BaseStruct, FCustomPropertyTypeLayoutMap()))
			{
				bCustomizedProperty = true;
			}
		}
	}

	if (bCustomizedProperty)
	{
		// Use the struct name instead of the fully-qualified property name
		const FText Label = BaseStruct->GetDisplayNameText();
		const FName PropertyName = StructProperty->GetProperty()->GetFName();

		// If the struct has a property customization, then we'll route through AddChildStructure, as it supports
		// IPropertyTypeCustomization. The other branch is mostly kept as-is for legacy support purposes.
		ChildBuilder.AddChildStructure(
			StructProperty.ToSharedRef(), NewStructProvider, PropertyName, Label
		);
	}
	else
	{
		StructProperty->RemoveChildren();
		TArray<TSharedPtr<IPropertyHandle>> ChildProperties = StructProperty->AddChildStructure(NewStructProvider);
		AddChildRows(ChildBuilder, ChildProperties);
	}

	bCanHandleStructValuePostChange = true;

	CachedInstanceTypes = GetInstanceTypes();
}

void FInstancedStructDataDetails::AddChildRows(IDetailChildrenBuilder& ChildBuilder, const TArray<TSharedPtr<IPropertyHandle>>& ChildProperties)
{
	// Properties may have Category metadata. If that's the case, they should be added under groups.
	TMap<TSharedPtr<IPropertyHandle>, IDetailGroup*> PropertyToGroup;
	GetPropertyGroups(ChildProperties, ChildBuilder, PropertyToGroup);

	for (TSharedPtr<IPropertyHandle> ChildHandle : ChildProperties)
	{
		IDetailGroup** PropertyGroupPtr = PropertyToGroup.Find(ChildHandle);
		IDetailGroup* PropertyGroup = PropertyGroupPtr ? *PropertyGroupPtr : nullptr;
		AddChildRow(ChildBuilder, ChildHandle, PropertyGroup);
	}
}

void FInstancedStructDataDetails::AddChildRow(IDetailChildrenBuilder& ChildBuilder, TSharedPtr<IPropertyHandle> ChildHandle, IDetailGroup* PropertyGroup)
{
	// If the property has a group, add it under the group. Otherwise, just add it normally via the builder.
	if (PropertyGroup)
	{
		IDetailPropertyRow& Row = PropertyGroup->AddPropertyRow(ChildHandle.ToSharedRef());
		OnChildRowAdded(Row);
	}
	else
	{
		IDetailPropertyRow& Row = ChildBuilder.AddProperty(ChildHandle.ToSharedRef());
		OnChildRowAdded(Row);
	}
}

void FInstancedStructDataDetails::Tick(float DeltaTime)
{
	// If the instance types change (e.g. due to selecting new struct type), we'll need to update the layout.
	TArray<TWeakObjectPtr<const UStruct>> InstanceTypes = GetInstanceTypes();
	if (bStructChangeDetected || InstanceTypes != CachedInstanceTypes)
	{
		bStructChangeDetected = false;
		OnRegenerateChildren.ExecuteIfBound();
	}
}

FName FInstancedStructDataDetails::GetName() const
{
	static const FName Name("InstancedStructDataDetails");
	return Name;
}


////////////////////////////////////

TSharedRef<IPropertyTypeCustomization> FInstancedStructDetails::MakeInstance()
{
	return MakeShared<FInstancedStructDetails>();
}

FInstancedStructDetails::~FInstancedStructDetails()
{
	if (OnObjectsReinstancedHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectsReinstanced.Remove(OnObjectsReinstancedHandle);
	}
}

void FInstancedStructDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities();

	OnObjectsReinstancedHandle = FCoreUObjectDelegates::OnObjectsReinstanced.AddSP(this, &FInstancedStructDetails::OnObjectsReinstanced);

	if (!StructProperty->HasMetaData(UE::StructUtils::Metadata::HideStructTypeName) || StructProperty->GetMetaData(UE::StructUtils::Metadata::HideStructTypeName) == TEXT("false"))
	{
		HeaderRow
			.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FInstancedStructDetails::OnCopyInstancedStruct)))
			.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FInstancedStructDetails::OnPasteInstancedStruct), FCanExecuteAction::CreateSP(this, &FInstancedStructDetails::CanPasteInstancedStruct)))
			.NameContent()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MinDesiredWidth(250.f)
			.VAlign(VAlign_Center)
			[
				SAssignNew(StructPicker, SInstancedStructPicker, StructProperty, PropUtils)
			]
			.IsEnabled(StructProperty->IsEditable());
	}
}

void FInstancedStructDetails::OnCopyInstancedStruct() const
{
	FString Value;
	StructProperty->GetValueAsFormattedString(Value, PPF_Copy);

	FPlatformApplicationMisc::ClipboardCopy(*Value);
}

bool FInstancedStructDetails::CanPasteInstancedStruct() const
{
	FString Value;
	FPlatformApplicationMisc::ClipboardPaste(Value);

	if (StructProperty && StructPicker && !Value.IsEmpty())
	{
		bool bCanPasteValue = true;
		FInstancedStruct TempInstancedStructValue;

		// This value actually a valid instanced struct?
		{
			const TCHAR* Buffer = *Value;
			bCanPasteValue = TempInstancedStructValue.ImportTextItem(Buffer, PPF_Copy, nullptr, nullptr);
		}

		// Is this value compatible with our property type?
		if (bCanPasteValue)
		{
			if (!StructPicker->CanChangeStructType())
			{
				// Can only apply the value if the new struct type matches the existing struct type of every instance
				StructProperty->EnumerateConstRawData([&bCanPasteValue, &TempInstancedStructValue](const void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
				{
					const FInstancedStruct* InstancedStruct = static_cast<const FInstancedStruct*>(RawData);
					bCanPasteValue = InstancedStruct && InstancedStruct->GetScriptStruct() == TempInstancedStructValue.GetScriptStruct();
					return bCanPasteValue;
				});
			}
			else if (const UScriptStruct* BaseScriptStruct = StructPicker->GetBaseScriptStruct())
			{
				// Can only apply the value if the new struct type is compatible with our base struct type, or the new struct type is null
				if (const UScriptStruct* TempInstancedScriptStruct = TempInstancedStructValue.GetScriptStruct())
				{
					bCanPasteValue = TempInstancedScriptStruct->IsChildOf(BaseScriptStruct);
				}
			}
		}

		return bCanPasteValue;
	}

	return false;
}

void FInstancedStructDetails::OnPasteInstancedStruct()
{
	FString Value;
	FPlatformApplicationMisc::ClipboardPaste(Value);

	FScopedTransaction Transaction(LOCTEXT("Paste", "Paste Property"));
	StructProperty->SetValueFromFormattedString(*Value, PPF_Copy);
}

void FInstancedStructDetails::OnObjectsReinstanced(const FReplacementObjectMap& ObjectMap)
{
	// Force update the details when BP is compiled, since we may cached hold references to the old object or class.
	if (!ObjectMap.IsEmpty() && PropUtils.IsValid())
	{
		PropUtils->RequestRefresh();
	}
}

void FInstancedStructDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedRef<FInstancedStructDataDetails> DataDetails = MakeShared<FInstancedStructDataDetails>(StructPropertyHandle);
	StructBuilder.AddCustomBuilder(DataDetails);
}

TSharedRef<IPropertyTypeCustomization> FInstancedStructArrayDetails::MakeInstance()
{
	return MakeShared<FInstancedStructArrayDetails>();
}

void FInstancedStructArrayDetails::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow,	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// Leaving empty so no header is displayed
}

void FInstancedStructArrayDetails::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{	
	TSharedPtr<IPropertyHandle> ValuesHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInstancedStructArray, Values));
	
	TSharedPtr<IPropertyHandleArray> ArrayHandle = ValuesHandle->AsArray();
	
	if (!ArrayHandle.IsValid())
	{
		return;
	}
	
	TSharedRef<FInstancedStructArrayBuilder> ArrayBuilder = MakeShared<FInstancedStructArrayBuilder>(ValuesHandle.ToSharedRef());
	ArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FInstancedStructArrayDetails::OnGenerateArrayElementWidget));
	ChildBuilder.AddCustomBuilder(ArrayBuilder);
}

void FInstancedStructArrayDetails::OnGenerateArrayElementWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 Index, IDetailChildrenBuilder& DetailChildrenBuilder)
{
	DetailChildrenBuilder.AddCustomBuilder(MakeShared<FInstancedStructDataWithHeaderDetails>(PropertyHandle));
}

#undef LOCTEXT_NAMESPACE
