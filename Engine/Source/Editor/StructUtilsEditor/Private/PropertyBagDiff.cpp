// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBagDiff.h"

#include "Containers/Map.h"
#include "StructUtils/PropertyBag.h"
#include "StructUtils/StructView.h"
#include "UObject/UnrealType.h"

namespace UE::StructUtils
{
	namespace Private
	{
		/** "Category" metadata key, broken out so CategoryChanged is reported separately from generic MetaDataChanged. */
		static const FName CategoryMetaKey(TEXT("Category"));
		
		/** Find a metadata entry by Key. Returns nullptr if absent. */
		static const FString* FindMetaValue(const FPropertyBagPropertyDesc& Desc, FName Key)
		{
			for (const FPropertyBagPropertyDescMetaData& Entry : Desc.MetaData)
			{
				if (Entry.Key == Key)
				{
					return &Entry.Value;
				}
			}
			return nullptr;
		}

		/** Compares the raw bytes of one property between two bags via the cached FProperty. Returns true if identical or if comparison can't be performed. */
		static bool PropertyValuesIdentical(const FInstancedPropertyBag& Old, const FPropertyBagPropertyDesc& OldDesc, const FInstancedPropertyBag& New, const FPropertyBagPropertyDesc& NewDesc)
		{
			const FProperty* OldProp = OldDesc.CachedProperty;
			const FProperty* NewProp = NewDesc.CachedProperty;
			if (!OldProp || !NewProp)
			{
				// Without resolved properties we cannot diff values; treat as identical to avoid spurious entries.
				return true;
			}

			const uint8* OldBase = Old.GetValue().GetMemory();
			const uint8* NewBase = New.GetValue().GetMemory();
			if (!OldBase || !NewBase)
			{
				return true;
			}

			const void* OldAddr = OldProp->ContainerPtrToValuePtr<void>(OldBase);
			const void* NewAddr = NewProp->ContainerPtrToValuePtr<void>(NewBase);
			return OldProp->Identical(OldAddr, NewAddr, PPF_None);
		}
	}

	void DiffPropertyBags(const FInstancedPropertyBag& Old, const FInstancedPropertyBag& New, TArray<FPropertyBagDiffEntry>& OutDiffs)
	{
		using namespace Private;

		const UPropertyBag* OldBagStruct = Old.GetPropertyBagStruct();
		const UPropertyBag* NewBagStruct = New.GetPropertyBagStruct();

		const TConstArrayView<FPropertyBagPropertyDesc> OldDescs = OldBagStruct ? OldBagStruct->GetPropertyDescs() : TConstArrayView<FPropertyBagPropertyDesc>{};
		const TConstArrayView<FPropertyBagPropertyDesc> NewDescs = NewBagStruct ? NewBagStruct->GetPropertyDescs() : TConstArrayView<FPropertyBagPropertyDesc>{};

		TMap<FGuid, const FPropertyBagPropertyDesc*> OldByID;
		OldByID.Reserve(OldDescs.Num());
		for (const FPropertyBagPropertyDesc& Desc : OldDescs)
		{
			OldByID.Add(Desc.ID, &Desc);
		}

		TMap<FGuid, const FPropertyBagPropertyDesc*> NewByID;
		NewByID.Reserve(NewDescs.Num());
		for (const FPropertyBagPropertyDesc& Desc : NewDescs)
		{
			NewByID.Add(Desc.ID, &Desc);
		}
		
		auto MakeDiff = [](const FPropertyBagPropertyDesc* OldDesc, const FPropertyBagPropertyDesc* NewDesc)
		{
			check(OldDesc || NewDesc);
			FPropertyBagDiffEntry Entry{};
			// To satisfy static analyzer, ternary operator was throwing a warning.
			if (OldDesc)
			{
				Entry.PropertyID = OldDesc->ID;
			}
			else
			{
				check(NewDesc);
				Entry.PropertyID = NewDesc->ID;
			}

			Entry.OldName = OldDesc ? OldDesc->Name : NAME_None;
			Entry.NewName = NewDesc ? NewDesc->Name : NAME_None;
			Entry.OldDesc = OldDesc;
			Entry.NewDesc = NewDesc;
			return Entry;
		};

		// Walk Old: detect Removed, Renamed, TypeChanged, FlagsChanged, CategoryChanged, MetaDataChanged, ValueChanged.
		for (const FPropertyBagPropertyDesc& OldDesc : OldDescs)
		{
			const FPropertyBagPropertyDesc* const* NewDescPtrPtr = NewByID.Find(OldDesc.ID);
			if (!NewDescPtrPtr)
			{
				FPropertyBagDiffEntry Entry = MakeDiff(&OldDesc, nullptr);
				Entry.DiffType = EPropertyBagDiffType::Removed;
				OutDiffs.Add(MoveTemp(Entry));
				continue;
			}

			const FPropertyBagPropertyDesc& NewDesc = **NewDescPtrPtr;
			
			auto MakeBase = [&MakeDiff, &OldDesc, &NewDesc]()
			{
				return MakeDiff(&OldDesc, &NewDesc);
			};
			
			if (OldDesc.Name != NewDesc.Name)
			{
				FPropertyBagDiffEntry Entry = MakeBase();
				Entry.DiffType = EPropertyBagDiffType::Renamed;
				OutDiffs.Add(MoveTemp(Entry));
			}
			
			const bool bTypeChanged = !OldDesc.CompatibleType(NewDesc);

			if (bTypeChanged)
			{
				FPropertyBagDiffEntry Entry = MakeBase();
				Entry.DiffType = EPropertyBagDiffType::TypeChanged;
				OutDiffs.Add(MoveTemp(Entry));
			}

			if (OldDesc.PropertyFlags != NewDesc.PropertyFlags)
			{
				FPropertyBagDiffEntry Entry = MakeBase();
				Entry.DiffType = EPropertyBagDiffType::FlagsChanged;
				OutDiffs.Add(MoveTemp(Entry));
			}
			
			// Diff metadata key by key. Category is reported as its own diff type.
			{
				TSet<FName> AllKeys;
				AllKeys.Reserve(OldDesc.MetaData.Num() + NewDesc.MetaData.Num());
				for (const FPropertyBagPropertyDescMetaData& Meta : OldDesc.MetaData)
				{
					AllKeys.Add(Meta.Key);
				}

				for (const FPropertyBagPropertyDescMetaData& Meta : NewDesc.MetaData)
				{
					AllKeys.Add(Meta.Key);
				}

				for (const FName Key : AllKeys)
				{
					const FString* OldVal = FindMetaValue(OldDesc, Key);
					const FString* NewVal = FindMetaValue(NewDesc, Key);
					const bool bDiffers = (OldVal == nullptr) != (NewVal == nullptr)
						|| (OldVal && NewVal && *OldVal != *NewVal);
					if (!bDiffers)
					{
						continue;
					}

					FPropertyBagDiffEntry Entry = MakeBase();
					if (Key == CategoryMetaKey)
					{
						Entry.DiffType = EPropertyBagDiffType::CategoryChanged;
					}
					else
					{
						Entry.DiffType = EPropertyBagDiffType::MetaDataChanged;
						Entry.MetaDataKey = Key;
					}
					OutDiffs.Add(MoveTemp(Entry));
				}
			}

			// No need to test for identical if type changed.
			if (!bTypeChanged && !PropertyValuesIdentical(Old, OldDesc, New, NewDesc))
			{
				FPropertyBagDiffEntry Entry = MakeBase();
				Entry.DiffType = EPropertyBagDiffType::ValueChanged;
				OutDiffs.Add(MoveTemp(Entry));
			}
		}

		// Walk New: detect Added.
		for (const FPropertyBagPropertyDesc& NewDesc : NewDescs)
		{
			if (!OldByID.Contains(NewDesc.ID))
			{
				FPropertyBagDiffEntry Entry = MakeDiff(nullptr, &NewDesc);
				Entry.DiffType = EPropertyBagDiffType::Added;
				OutDiffs.Add(MoveTemp(Entry));
			}
		}

		// Reordering: build the sequence of IDs that survive in both bags, in each bag's order,
		// and emit Reordered for IDs whose position differs.
		TArray<FGuid> OldOrderedIDs;
		OldOrderedIDs.Reserve(OldDescs.Num());
		for (const FPropertyBagPropertyDesc& Desc : OldDescs)
		{
			if (NewByID.Contains(Desc.ID))
			{
				OldOrderedIDs.Add(Desc.ID);
			}
		}

		TMap<FGuid, int32> NewOrderedIndexByID;
		NewOrderedIndexByID.Reserve(OldOrderedIDs.Num());
		int32 NewOrderedIdx = 0;
		for (const FPropertyBagPropertyDesc& Desc : NewDescs)
		{
			if (OldByID.Contains(Desc.ID))
			{
				NewOrderedIndexByID.Add(Desc.ID, NewOrderedIdx++);
			}
		}

		for (int32 OldOrderedIdx = 0; OldOrderedIdx < OldOrderedIDs.Num(); ++OldOrderedIdx)
		{
			const FGuid& ID = OldOrderedIDs[OldOrderedIdx];
			const int32 NewIdx = NewOrderedIndexByID.FindChecked(ID);
			if (NewIdx != OldOrderedIdx)
			{
				const FPropertyBagPropertyDesc* OldDesc = OldByID.FindChecked(ID);
				const FPropertyBagPropertyDesc* NewDesc = NewByID.FindChecked(ID);

				FPropertyBagDiffEntry Entry = MakeDiff(OldDesc, NewDesc);
				Entry.DiffType = EPropertyBagDiffType::Reordered;
				OutDiffs.Add(MoveTemp(Entry));
			}
		}
	}
}
