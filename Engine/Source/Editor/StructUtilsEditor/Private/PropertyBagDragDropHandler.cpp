// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBagDragDropHandler.h"

#include "DataHierarchyViewModelBase.h"
#include "IPropertyBagEditorGraph.h"
#include "PropertyBagDetails.h"
#include "StructUtilsMetadata.h"
#include "EdGraph/EdGraph.h"
#include "Input/Reply.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "StructUtilsEditor"

namespace PropertyBagDragDrop::Constants
{
	const FText DefaultValidTargetText = LOCTEXT("ValidTargetTooltip", "Drop property here");
	const FText DefaultSourceIsTargetText = LOCTEXT("SourceIsTargetTooltip", "Choose a different target");
	const FText DefaultInvalidTargetText = LOCTEXT("InvalidTargetTooltip", "Invalid target");
}

FPropertyBagDetailsDragDropOp::FDecoration::FDecoration(const FText& InMessage, const FSlateBrush* InIcon, const FLinearColor& InColor)
	: Message(InMessage)
	, Icon(InIcon)
	, IconColor(InColor) {}

FPropertyBagDetailsDragDropOp::FPropertyBagDetailsDragDropOp(const FPropertyBagPropertyDesc& InPropertyDesc)
	: PropertyDesc(InPropertyDesc)
{
	FEdGraphPinType PinType = UE::StructUtils::GetPropertyDescAsPin(InPropertyDesc);

	TSharedRef<SWidget> DefaultDecorator = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.Padding(2.f)
	[
		SNew(SImage)
		.Image(FBlueprintEditorUtils::GetIconFromPin(PinType))
		.ColorAndOpacity(GetDefault<UPropertyBagSchema>()->GetPinTypeColor(PinType))
	]
	+ SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	[
		SNew(STextBlock).Text(FText::FromName(InPropertyDesc.Name))
	];

	FGraphEditorDragDropAction::Construct();

	SetFeedbackMessage(DefaultDecorator);
}

void FPropertyBagDetailsDragDropOp::SetDecoration(const EPropertyBagDropState NewDropState, TOptional<FDecoration> OverriddenDecoration)
{
	if (CurrentDropState == NewDropState)
	{
		return;
	}

	FText Message;
	const FSlateBrush* Icon = nullptr;
	FLinearColor Color = FLinearColor::White;

	if (OverriddenDecoration)
	{
		Icon = OverriddenDecoration->Icon;
		Message = OverriddenDecoration->Message;
		Color = OverriddenDecoration->IconColor;
		bDropTargetValid = NewDropState == EPropertyBagDropState::Valid;
	}
	else if (NewDropState == EPropertyBagDropState::Valid)
	{
		Icon = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.OK");
		Message = PropertyBagDragDrop::Constants::DefaultValidTargetText;
		bDropTargetValid = true;
	}
	else if (NewDropState == EPropertyBagDropState::SourceIsTarget)
	{
		Icon = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.OKWarn");
		Color = FLinearColor::White.CopyWithNewOpacity(0.5f);
		Message = PropertyBagDragDrop::Constants::DefaultSourceIsTargetText;
		bDropTargetValid = false;
	}
	else // DragDropTarget == EPropertyBagDragDropTargetState::Invalid
	{
		check(NewDropState == EPropertyBagDropState::Invalid);
		Icon = FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.Error");
		Message = PropertyBagDragDrop::Constants::DefaultSourceIsTargetText;
		bDropTargetValid = false;
	}

	SetSimpleFeedbackMessage(Icon, Color, Message);
	SetDecoratorVisibility(true);

	CurrentDropState = NewDropState;
}

void FPropertyBagDetailsDragDropOp::HoverTargetChanged()
{
	IPropertyBagEdGraphDragAndDrop* Graph = Cast<IPropertyBagEdGraphDragAndDrop>(GetHoveredGraph());
	if (!Graph)
	{
		SetDecoration(EPropertyBagDropState::Invalid);
		SetDecoratorVisibility(false);
		return;
	}

	bool bIsValid = false;
	// Must check in order as the getters will escalate up the chain. Check the pin first.
	if (const UEdGraphPin* Pin = GetHoveredPin())
	{
		bIsValid = Graph->CanReceivePropertyBagDetailsDropOnGraphPin(Pin);
	} // Check the node next.
	else if (const UEdGraphNode* Node = GetHoveredNode())
	{
		bIsValid = Graph->CanReceivePropertyBagDetailsDropOnGraphNode(Node);
	}
	else // Finally, check the graph.
	{
		bIsValid = Graph->CanReceivePropertyBagDetailsDropOnGraph(CastChecked<UEdGraph>(Graph));
	}

	SetDecoration(bIsValid ? EPropertyBagDropState::Valid : EPropertyBagDropState::Invalid);
}

EVisibility FPropertyBagDetailsDragDropOp::GetIconVisible() const
{
	return EVisibility::Visible;
}

EVisibility FPropertyBagDetailsDragDropOp::GetErrorIconVisible() const
{
	// Error icon handled by the decorator directly
	return EVisibility::Collapsed;
}

FReply FPropertyBagDetailsDragDropOp::DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	if (UEdGraphPin* Pin = GetHoveredPin())
	{
		const IPropertyBagEdGraphDragAndDrop* PropertyBagGraph = GetPropertyBagEdGraphDragAndDropInterface();
		if (PropertyBagGraph && PropertyBagGraph->CanReceivePropertyBagDetailsDropOnGraphPin(Pin))
		{
			return PropertyBagGraph->OnPropertyBagDetailsDropOnGraphPin(PropertyDesc, Pin, GraphPosition);
		}
	}

	return FReply::Handled();
}

FReply FPropertyBagDetailsDragDropOp::DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	if (UEdGraphNode* Node = GetHoveredNode())
	{
		const IPropertyBagEdGraphDragAndDrop* PropertyBagGraph = GetPropertyBagEdGraphDragAndDropInterface();
		if (PropertyBagGraph && PropertyBagGraph->CanReceivePropertyBagDetailsDropOnGraphNode(Node))
		{
			// @todo_pcg: remove return type?
			return PropertyBagGraph->OnPropertyBagDetailsDropOnGraphNode(PropertyDesc, Node, GraphPosition);
		}
	}

	return FReply::Handled();
}

FReply FPropertyBagDetailsDragDropOp::DroppedOnPanel(const TSharedRef<SWidget>& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	check(&Graph == GetHoveredGraph())
	const IPropertyBagEdGraphDragAndDrop* PropertyBagGraph = GetPropertyBagEdGraphDragAndDropInterface();
	if (PropertyBagGraph && PropertyBagGraph->CanReceivePropertyBagDetailsDropOnGraph(&Graph))
	{
		return PropertyBagGraph->OnPropertyBagDetailsDropOnGraph(PropertyDesc, &Graph, GraphPosition);
	}

	return FReply::Handled();
}

void FPropertyBagDetailsDragDropOp::SetHierarchyElement(TSharedPtr<FHierarchyElementViewModel> InElement)
{
	HierarchyElement = InElement;
}

TSharedPtr<FHierarchyElementViewModel> FPropertyBagDetailsDragDropOp::GetHierarchyElement() const
{
	return HierarchyElement.Pin();
}

IPropertyBagEdGraphDragAndDrop* FPropertyBagDetailsDragDropOp::GetPropertyBagEdGraphDragAndDropInterface() const
{
	UEdGraph* Graph = GetHoveredGraph();
	return Graph ? Cast<IPropertyBagEdGraphDragAndDrop>(Graph) : nullptr;
}

FPropertyBagDetailsDragDropHandlerTarget::FPropertyBagDetailsDragDropHandlerTarget(const FCanAcceptPropertyBagDetailsRowDropOp& CanAcceptDragDrop, const FOnPropertyBagDetailsRowDropOp& OnDragDrop)
	: CanAcceptDetailsRowDropOp(CanAcceptDragDrop)
	, OnHandleDetailsRowDropOp(OnDragDrop) {}

TSharedPtr<FDragDropOperation> FPropertyBagDetailsDragDropHandlerTarget::CreateDragDropOperation() const
{
	return nullptr;
}

void FPropertyBagDetailsDragDropHandlerTarget::BindCanAcceptDragDrop(FCanAcceptPropertyBagDetailsRowDropOp&& CanAcceptDragDrop)
{
	CanAcceptDetailsRowDropOp = MoveTemp(CanAcceptDragDrop);
}

void FPropertyBagDetailsDragDropHandlerTarget::BindOnHandleDragDrop(FOnPropertyBagDetailsRowDropOp&& OnDragDrop)
{
	OnHandleDetailsRowDropOp = MoveTemp(OnDragDrop);
}

void FPropertyBagDetailsDragDropHandlerTarget::BindOnHandleHierarchyElementDrop(FOnPropertyBagHierarchyElementDropOp&& OnDrop)
{
	OnHandleHierarchyElementDropOp = MoveTemp(OnDrop);
}

TOptional<EItemDropZone> FPropertyBagDetailsDragDropHandlerTarget::CanAcceptDrop(const FDragDropEvent& DragDropSource, const EItemDropZone DropZone) const
{
	// Property Bag Details Drag and Drop Handler
	if (const TSharedPtr<FPropertyBagDetailsDragDropOp> DropOp = DragDropSource.GetOperationAs<FPropertyBagDetailsDragDropOp>())
	{
		if (CanAcceptDetailsRowDropOp.IsBound())
		{
			return CanAcceptDetailsRowDropOp.Execute(DropOp, DropZone);
		}
	}

	return TOptional<EItemDropZone>();
}

bool FPropertyBagDetailsDragDropHandlerTarget::AcceptDrop(const FDragDropEvent& DragDropSource, const EItemDropZone DropZone) const
{
	if (const TSharedPtr<FPropertyBagDetailsDragDropOp> DropOp = DragDropSource.GetOperationAs<FPropertyBagDetailsDragDropOp>())
	{
		// Try hierarchy element path first — gets the element directly from the op without a PropertyDesc identity roundtrip
		if (OnHandleHierarchyElementDropOp.IsBound())
		{
			if (TSharedPtr<FHierarchyElementViewModel> Element = DropOp->GetHierarchyElement())
			{
				return OnHandleHierarchyElementDropOp.Execute(Element, DropZone).IsEventHandled();
			}
			
			// The view model should always be valid but in case it's not, something went wrong; don't accept the drop.
			return false;
		}

		// Regular property desc path for non-hierarchy property bags
		if (OnHandleDetailsRowDropOp.IsBound())
		{
			return OnHandleDetailsRowDropOp.Execute(DropOp->PropertyDesc, DropZone).IsEventHandled();
		}
	}

	return false;
}

FPropertyBagDetailsDragDropHandler::FPropertyBagDetailsDragDropHandler(const FPropertyBagPropertyDesc& InPropertyDesc)
	: PropertyDesc(InPropertyDesc) {}

TSharedPtr<FDragDropOperation> FPropertyBagDetailsDragDropHandler::CreateDragDropOperation() const
{
	if (TSharedPtr<FPropertyBagDetailsDragDropOp> DragOp = MakeShared<FPropertyBagDetailsDragDropOp>(PropertyDesc))
	{
		DragOp->SetHierarchyElement(HierarchyElement.Pin());
		return DragOp;
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE