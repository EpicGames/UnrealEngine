// Copyright Epic Games, Inc. All Rights Reserved.

#include "SInstancedStructPicker.h"

#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "IPropertyUtilities.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "StructUtilsEditorUtilsPrivate.h"
#include "StructUtilsMetadata.h"
#include "Modules/ModuleManager.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/InstancedStructBaseStructQueryParams.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "StructUtilsEditor"

namespace UE::StructUtils::Private
{
	FPropertyAccess::Result GetCommonScriptStruct(const TSharedPtr<IPropertyHandle>& StructProperty,
		const UScriptStruct*& OutCommonStruct)
	{
		bool bHasResult = false;
		bool bHasMultipleValues = false;

		StructProperty->EnumerateConstRawData([&OutCommonStruct, &bHasResult, &bHasMultipleValues](const void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
		{
			if (const FInstancedStruct* InstancedStruct = static_cast<const FInstancedStruct*>(RawData))
			{
				const UScriptStruct* Struct = InstancedStruct->GetScriptStruct();

				if (!bHasResult)
				{
					OutCommonStruct = Struct;
				}
				else if (OutCommonStruct != Struct)
				{
					bHasMultipleValues = true;
				}

				bHasResult = true;
			}

			return true;
		});

		if (bHasMultipleValues)
		{
			return FPropertyAccess::MultipleValues;
		}

		return bHasResult ? FPropertyAccess::Success : FPropertyAccess::Fail;
	}
} // UE::StructUtils::Private

////////////////////////////////////

bool FInstancedStructFilter::IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs)
{
	bool bStructAllowed = true;
	if (!AllowedStructs.IsEmpty())
	{
		bStructAllowed = false;
		for (const TSoftObjectPtr<const UScriptStruct>& AllowedStruct : AllowedStructs)
		{
			if (bAllowChildStructs ? InStruct->IsChildOf(AllowedStruct.Get()) : InStruct == AllowedStruct)
			{
				bStructAllowed = true;
				break;
			}
		}
	}
	if (!DisallowedStructs.IsEmpty())
	{
		for (const TSoftObjectPtr<const UScriptStruct>& DisallowedStruct : DisallowedStructs)
		{
			if (bAllowChildStructs ? InStruct->IsChildOf(DisallowedStruct.Get()) : DisallowedStruct == InStruct)
			{
				bStructAllowed = false;
				break;
			}
		}
	}

	if (!bStructAllowed)
	{
		return false;
	}

	if (InStruct->IsA<UUserDefinedStruct>())
	{
		return bAllowUserDefinedStructs;
	}

	static const FName NAME_HiddenMetaTag = "Hidden";
	if (InStruct->HasMetaData(NAME_HiddenMetaTag))
	{
		return false;
	}

	if (InStruct == BaseStruct)
	{
		return bAllowBaseStruct;
	}

	// If we have a base struct, make sure the candidate struct is a child
	if (BaseStruct.IsValid() && !InStruct->IsChildOf(BaseStruct.Get()))
	{
		return false;
	}

	if (AssetReferenceFilter.IsValid())
	{
		if (!AssetReferenceFilter->PassesFilter(FAssetData(InStruct)))
		{
			return false;
		}
	}
	
	return true;
}

bool FInstancedStructFilter::IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs)
{
	// User Defined Structs don't support inheritance, so only include them requested
	return bAllowUserDefinedStructs;
}

////////////////////////////////////

void SInstancedStructPicker::Construct(const FArguments& InArgs, TSharedPtr<IPropertyHandle> InStructProperty,
	TSharedPtr<IPropertyUtilities> InPropertyUtils)
{
	OnStructPicked = InArgs._OnStructPicked;
	AllowedStructs = InArgs._AllowedStructs;
	DisallowedStructs = InArgs._DisallowedStructs;
	StructProperty = MoveTemp(InStructProperty);
	PropUtils = MoveTemp(InPropertyUtils);

	if (!StructProperty.IsValid() || !PropUtils.IsValid())
	{
		return;
	}

	bCanChangeStructType = !StructProperty->HasMetaData(UE::StructUtils::Metadata::StructTypeConstName);

	ChildSlot
	[
		SAssignNew(ComboButton, SComboButton)
		.OnGetMenuContent(this, &SInstancedStructPicker::GenerateStructPicker)
		.ContentPadding(0)
		.IsEnabled(StructProperty->IsEditable() && bCanChangeStructType)
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(SImage)
				.Image(this, &SInstancedStructPicker::GetDisplayValueIcon)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SInstancedStructPicker::GetDisplayValueString)
				.ToolTipText(this, &SInstancedStructPicker::GetTooltipText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
	];
}

const UScriptStruct* SInstancedStructPicker::GetBaseScriptStruct() const
{
	using namespace UE::StructUtils::Private;
	
	const FString& BaseStructFunctionName = StructProperty->GetMetaData(UE::StructUtils::Metadata::BaseStructFunctionName);
	if (!BaseStructFunctionName.IsEmpty())
	{
		if (TOptional<FFindUserFunctionResult> Result = FindUserFunction(StructProperty,UE::StructUtils::Metadata::BaseStructFunctionName); Result.IsSet())
		{
			check(Result.GetValue().Function && Result.GetValue().Target);

			DECLARE_DELEGATE_RetVal_OneParam(TObjectPtr<UScriptStruct>, FGetBaseStruct, UE::StructUtils::FInstancedStructBaseStructQueryParams);

			UE::StructUtils::FInstancedStructBaseStructQueryParams Parameters{};

			// Go up the hierarchy while we have properties
			TSharedPtr<IPropertyHandle> CurrHandle = StructProperty;
			while (CurrHandle.IsValid())
			{
				if (const FProperty* Property = CurrHandle->GetProperty())
				{
					Parameters.PropertyChainWithArrayIndex.Emplace(Property, CurrHandle->GetArrayIndex());
				}
				
				CurrHandle = CurrHandle->GetParentHandle();
			}

			Algo::Reverse(Parameters.PropertyChainWithArrayIndex);
			
			if (const UScriptStruct* BaseStruct = FGetBaseStruct::CreateUFunction(Result.GetValue().Target, Result.GetValue().Function->GetFName()).Execute(Parameters))
			{
				return BaseStruct;
			}
		}
	}

	const FString& BaseStructName = StructProperty->GetMetaData(UE::StructUtils::Metadata::BaseStructName);
	if (!BaseStructName.IsEmpty())
	{
		if (UScriptStruct* Struct = UClass::TryFindTypeSlow<UScriptStruct>(BaseStructName))
		{
			return Struct;
		}

		// TODO: We can't load an object at that point, this would break autoRTFM in Editor.
		// Should probably have some kind of UI feedback to indicate that we are waiting.
		// return LoadObject<UScriptStruct>(nullptr, *BaseStructName);
		return nullptr;
	}
	else
	{
		return nullptr;
	}
}

FText SInstancedStructPicker::GetDisplayValueString() const
{
	const UScriptStruct* CommonStruct = nullptr;
	const FPropertyAccess::Result Result = UE::StructUtils::Private::GetCommonScriptStruct(StructProperty, CommonStruct);

	if (Result == FPropertyAccess::Success)
	{
		if (CommonStruct)
		{
			return CommonStruct->GetDisplayNameText();
		}
		return LOCTEXT("NullScriptStruct", "None");
	}
	if (Result == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return FText::GetEmpty();
}

FText SInstancedStructPicker::GetTooltipText() const
{
	const UScriptStruct* CommonStruct = nullptr;
	const FPropertyAccess::Result Result = UE::StructUtils::Private::GetCommonScriptStruct(StructProperty, CommonStruct);

	if (CommonStruct && Result == FPropertyAccess::Success)
	{
		return CommonStruct->GetToolTipText();
	}

	return GetDisplayValueString();
}

const FSlateBrush* SInstancedStructPicker::GetDisplayValueIcon() const
{
	const UScriptStruct* CommonStruct = nullptr;
	if (UE::StructUtils::Private::GetCommonScriptStruct(StructProperty, CommonStruct) == FPropertyAccess::Success)
	{
		return FSlateIconFinder::FindIconBrushForClass(CommonStruct, "ClassIcon.Object");
	}

	return nullptr;
}

TSharedRef<SWidget> SInstancedStructPicker::GenerateStructPicker()
{
	static const FName NAME_ExcludeBaseStruct = "ExcludeBaseStruct";
	static const FName NAME_ExcludeChildStructs = "ExcludeChildStructs";
	static const FName NAME_HideViewOptions = "HideViewOptions";
	static const FName NAME_ShowTreeView = "ShowTreeView";

	const bool bExcludeBaseStruct = StructProperty->HasMetaData(NAME_ExcludeBaseStruct);
	const bool bExcludeChildStructs = StructProperty->HasMetaData(NAME_ExcludeChildStructs);
	const bool bAllowNone = !(StructProperty->GetMetaDataProperty()->PropertyFlags & CPF_NoClear);
	const bool bHideViewOptions = StructProperty->HasMetaData(NAME_HideViewOptions);
	const bool bShowTreeView = StructProperty->HasMetaData(NAME_ShowTreeView);

	const UScriptStruct* BaseScriptStruct = GetBaseScriptStruct();

	TSharedRef<FInstancedStructFilter> StructFilter = MakeShared<FInstancedStructFilter>();
	StructFilter->BaseStruct = BaseScriptStruct;
	StructFilter->bAllowUserDefinedStructs = BaseScriptStruct == nullptr; // Only allow user defined structs when BaseStruct is not set.
	StructFilter->bAllowBaseStruct = !bExcludeBaseStruct;
	StructFilter->bAllowChildStructs = !bExcludeChildStructs;

	if (GEditor && StructProperty)
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;

		TArray<UPackage*> OuterPackages;
		StructProperty->GetOuterPackages(OuterPackages);
		for (UPackage* OuterPackage : OuterPackages)
		{
			AssetReferenceFilterContext.AddReferencingAsset(FAssetData(OuterPackage));
		}

		StructFilter->AssetReferenceFilter = GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);

		auto SoftPointerTransform = [](const UScriptStruct* InStruct) -> TSoftObjectPtr<const UScriptStruct>
		{
			return InStruct;
		};

		Algo::Transform(PropertyCustomizationHelpers::GetStructsFromMetadataString(StructProperty->GetMetaData("AllowedClasses")), StructFilter->AllowedStructs, SoftPointerTransform);
		Algo::Transform(PropertyCustomizationHelpers::GetStructsFromMetadataString(StructProperty->GetMetaData("DisallowedClasses")), StructFilter->DisallowedStructs, SoftPointerTransform);

		StructFilter->AllowedStructs.Append(AllowedStructs);
		StructFilter->DisallowedStructs.Append(DisallowedStructs);

		TArray<UObject*> OwningObjects;
		StructProperty->GetOuterObjects(OwningObjects);

		DECLARE_DELEGATE_RetVal(TArray<TSoftObjectPtr<UScriptStruct>>, FGetClasses);

		auto GatherStructs = [this, &OwningObjects](const FName MetaName, TArray<TSoftObjectPtr<const UScriptStruct>>& OutStructs)
		{
			const FString& FunctionName = StructProperty->GetMetaData(MetaName);
			if (!FunctionName.IsEmpty())
			{
				// First look for a library function assuming fully qualified path, e.g. /Script/ModuleName.ClassName:FunctionName
				if (FunctionName.Contains(TEXT(".")))
				{
					const UFunction* Function = FindObject<UFunction>(nullptr, *FunctionName, EFindObjectFlags::ExactClass);
					if (ensureMsgf(Function && Function->HasAnyFunctionFlags(EFunctionFlags::FUNC_Static), TEXT("Invalid %s: %s"), *MetaName.ToString(), *FunctionName))
					{
						UObject* FunctionCDO = Function->GetOwnerClass()->GetDefaultObject();
						OutStructs.Append(FGetClasses::CreateUFunction(FunctionCDO, Function->GetFName()).Execute());
					}
				}
				else
				{
					for (UObject* OwningObject : OwningObjects)
					{
						const UFunction* Function = OwningObject ? OwningObject->FindFunction(*FunctionName) : nullptr;
						if (Function != nullptr)
						{
							OutStructs.Append(FGetClasses::CreateUFunction(OwningObject, Function->GetFName()).Execute());
						}
					}
				}
			}
		};

		GatherStructs("GetAllowedClasses", StructFilter->AllowedStructs);
		GatherStructs("GetDisallowedClasses", StructFilter->DisallowedStructs);
	}
	else
	{
		StructFilter->AllowedStructs = AllowedStructs;
		StructFilter->DisallowedStructs = DisallowedStructs;
	}

	const UScriptStruct* SelectedStruct = nullptr;
	UE::StructUtils::Private::GetCommonScriptStruct(StructProperty, SelectedStruct);

	FStructViewerInitializationOptions Options;
	Options.bShowNoneOption = bAllowNone;
	Options.StructFilter = StructFilter;
	Options.NameTypeToDisplay = EStructViewerNameTypeToDisplay::DisplayName;
	Options.DisplayMode = bShowTreeView ? EStructViewerDisplayMode::TreeView : EStructViewerDisplayMode::ListView;
	Options.bAllowViewOptions = !bHideViewOptions;
	Options.SelectedStruct = SelectedStruct;
	Options.PropertyHandle = StructProperty;

	const FOnStructPicked OnPicked(FOnStructPicked::CreateSP(this, &SInstancedStructPicker::StructPicked));

	return SNew(SBox)
		.WidthOverride(280)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(500)
			[
				FModuleManager::LoadModuleChecked<FStructViewerModule>("StructViewer").CreateStructViewer(Options, OnPicked)
			]
		];
}

void SInstancedStructPicker::StructPicked(const UScriptStruct* InStruct)
{
	if (StructProperty && StructProperty->IsValidHandle())
	{
		FScopedTransaction Transaction(LOCTEXT("OnStructPicked", "Set Struct"));

		TArray<UObject*> OuterObjects;
		OuterObjects.Reserve(StructProperty->GetNumOuterObjects());
		StructProperty->GetOuterObjects(OuterObjects);

		if (OuterObjects.IsEmpty())
		{
			StructProperty->NotifyPreChange();

			StructProperty->EnumerateRawData([InStruct](void* RawData, const int32 /*DataIndex*/, const int32 /*NumDatas*/)
			{
				if (FInstancedStruct* InstancedStruct = static_cast<FInstancedStruct*>(RawData))
				{
					InstancedStruct->InitializeAs(InStruct);
				}
				return true;
			});

			StructProperty->NotifyPostChange(EPropertyChangeType::ValueSet);
			StructProperty->NotifyFinishedChangingProperties();
		}
		else
		{
			FInstancedStruct InstancedStruct;
			InstancedStruct.InitializeAs(InStruct);

			TArray<FString> NewValues;
			NewValues.Reserve(OuterObjects.Num());
			for (UObject* OuterObject : OuterObjects)
			{
				if (!OuterObject)
				{
					continue;
				}

				InstancedStruct.ExportTextItem(
					NewValues.Emplace_GetRef(),
					FInstancedStruct(),
					OuterObject,
					PPF_None,
					OuterObject->GetOutermost());
			}

			// This will internally handle propagation for values within default/archetype objects
			StructProperty->SetPerObjectValues(NewValues, EPropertyValueSetFlags::NotTransactable);
		}

		// After the type has changed, let's expand, so that the user can edit the newly appeared child properties
		StructProperty->SetExpanded(true);

		// Property tree will be invalid after changing the struct type, force update.
		if (PropUtils.IsValid())
		{
			PropUtils->RequestRefresh();
		}
	}

	ComboButton->SetIsOpen(false);
	OnStructPicked.ExecuteIfBound(InStruct);
}

#undef LOCTEXT_NAMESPACE
