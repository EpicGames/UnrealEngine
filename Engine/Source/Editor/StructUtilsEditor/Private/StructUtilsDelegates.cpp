// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtilsDelegates.h"

namespace UE::StructUtils::Delegates
{
#if WITH_EDITOR
	FOnUserDefinedStructReinstanced OnUserDefinedStructReinstanced;
#endif
} // UE::StructUtils::Delegates
