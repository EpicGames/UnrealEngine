// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtilsEditorModule.h"
#include "CoreGlobals.h"
#include "GameFramework/Actor.h"
#include "HierarchyEditor/PropertyBagHierarchyViewModel.h"
#include "Modules/ModuleManager.h"
#include "PropertyBagDetails.h"
#include "PropertyEditorModule.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/InstancedStructContainer.h"
#include "StructUtils/UserDefinedStruct.h"
#include "StructUtilsDelegates.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"

#define LOCTEXT_NAMESPACE "StructUtilsEditor"

IMPLEMENT_MODULE(FStructUtilsEditorModule, StructUtilsEditor)

namespace UE::StructUtilsEditor::Private
{
FName FInstancedStructTypeName;
FName FInstancedPropertyBagTypeName;
FName FInstancedStructArrayTypeName;
}

void FStructUtilsEditorModule::StartupModule()
{
	using namespace UE::StructUtilsEditor::Private;
	FInstancedStructTypeName = FInstancedStruct::StaticStruct()->GetFName();
	FInstancedPropertyBagTypeName = FInstancedPropertyBag::StaticStruct()->GetFName();
	FInstancedStructArrayTypeName = FInstancedStructArray::StaticStruct()->GetFName();

	// Register the details customizer
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout(FInstancedStructTypeName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FInstancedStructDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(FInstancedPropertyBagTypeName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPropertyBagDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(FInstancedStructArrayTypeName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FInstancedStructArrayDetails::MakeInstance));
	PropertyModule.NotifyCustomizationModuleChanged();
	
	HierarchyViewModelPropertyEditorPolicy = MakeUnique<FPropertyBagHierarchyViewModelPropertyEditorPolicy>();

	PostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FStructUtilsEditorModule::OnPostGarbageCollect);
}

void FStructUtilsEditorModule::ShutdownModule()
{
	using namespace UE::StructUtilsEditor::Private;

	FCoreUObjectDelegates::GetPostGarbageCollect().Remove(PostGarbageCollectHandle);
	HierarchyViewModelMap.Empty();

	HierarchyViewModelPropertyEditorPolicy.Reset();
	
	// Unregister the details customization
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout(FInstancedStructTypeName);
		PropertyModule.UnregisterCustomPropertyTypeLayout(FInstancedPropertyBagTypeName);
		PropertyModule.UnregisterCustomPropertyTypeLayout(FInstancedStructArrayTypeName);
		PropertyModule.NotifyCustomizationModuleChanged();
	}
}

FPropertyBagHierarchyViewModelOwner::FPropertyBagHierarchyViewModelOwner(UPropertyBagHierarchyViewModel* InViewModel)
	: ViewModel(InViewModel)
{
}

FPropertyBagHierarchyViewModelOwner::~FPropertyBagHierarchyViewModelOwner()
{
	// During GExitPurge UObjects are torn down in arbitrary order and may already be mid-destruction;
	// after UObjectBaseShutdown the GUObjectArray that MarkAsGarbage indexes into is gone. The inner
	// TStrongObjectPtr handles its own release safely in both cases, so just skip the explicit
	// Finalize/MarkAsGarbage when the UObject subsystem isn't in a state where they're meaningful.
	if (GExitPurge || !UObjectInitialized())
	{
		return;
	}

	if (UPropertyBagHierarchyViewModel* VM = ViewModel.Get())
	{
		// Finalize first so IsTickable() flips off before the strong ref drops; MarkAsGarbage then
		// makes the VM eligible for the next GC pass.
		VM->Finalize();
		VM->MarkAsGarbage();
	}
}

TSharedRef<FPropertyBagHierarchyViewModelOwner> FStructUtilsEditorModule::AcquireHierarchyViewModel(
	UPropertyBagHierarchyRoot* InHierarchyRoot,
	UObject* InOwningObject,
	TSharedRef<FPropertyPath> InPropertyPath,
	TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	if (TWeakPtr<FPropertyBagHierarchyViewModelOwner>* Existing = HierarchyViewModelMap.Find(InHierarchyRoot))
	{
		if (TSharedPtr<FPropertyBagHierarchyViewModelOwner> ExistingOwner = Existing->Pin())
		{
			// Refresh property utilities so RebuildDetails() works with the current details panel.
			ExistingOwner->Get()->SetPropertyUtilities(PropertyUtilities);
			return ExistingOwner.ToSharedRef();
		}
	}

	// Create new - publish the map entry BEFORE Initialize() to handle reentrancy.
	// Initialize() spins up a PropertyRowGenerator that triggers a details-layout rebuild,
	// which re-enters AcquireHierarchyViewModel. The map entry lets that reentrant call join
	// as an additional sharer of the same owner instead of constructing a second view model.
	UPropertyBagHierarchyViewModel* NewVM = NewObject<UPropertyBagHierarchyViewModel>(GetTransientPackage());
	TSharedRef<FPropertyBagHierarchyViewModelOwner> NewOwner = MakeShared<FPropertyBagHierarchyViewModelOwner>(NewVM);
	HierarchyViewModelMap.Add(InHierarchyRoot, NewOwner);
	NewVM->Initialize(InOwningObject, InPropertyPath);
	NewVM->SetPropertyUtilities(PropertyUtilities);
	return NewOwner;
}

void FStructUtilsEditorModule::OnPostGarbageCollect()
{
	for (auto It = HierarchyViewModelMap.CreateIterator(); It; ++It)
	{
		if (!It->Key.IsValid() || !It->Value.IsValid())
		{
			It.RemoveCurrent();
		}
	}
}

void FStructUtilsEditorModule::PreChange(const UUserDefinedStruct* StructToReinstantiate, FStructureEditorUtils::EStructureEditorChangeInfo Info)
{
}

void FStructUtilsEditorModule::PostChange(const UUserDefinedStruct* StructToReinstantiate, FStructureEditorUtils::EStructureEditorChangeInfo Info)
{
	if (!StructToReinstantiate)
	{
		return;
	}

	if (UE::StructUtils::Delegates::OnUserDefinedStructReinstanced.IsBound())
	{
		UE::StructUtils::Delegates::OnUserDefinedStructReinstanced.Broadcast(*StructToReinstantiate);
	}
}

#undef LOCTEXT_NAMESPACE
