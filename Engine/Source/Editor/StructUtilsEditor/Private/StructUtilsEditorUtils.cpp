// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtilsEditorUtils.h"

#include "StructUtilsEditorUtilsPrivate.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

#include "Blueprint/BlueprintSupport.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "PropertyBagDetails.h"

namespace UE::StructUtils::Private
{
	const FName NAME_GeneratePropertyWithoutID = "GeneratePropertyWithoutID";

	TNotNull<UUserDefinedStruct*> CreateUserDefinedStruct(TNotNull<UObject*> InOuter, FName Name, const FCreateUserDefinedStructArgs& Args)
	{
		UUserDefinedStruct* Struct = NewObject<UUserDefinedStruct>(InOuter, Name, Args.UserDefinedStructFlags);
		check(Struct);

		UUserDefinedStructEditorData* UDSEditorData = NewObject<UUserDefinedStructEditorData>(Struct, FName(), RF_Transactional);
		Struct->EditorData = UDSEditorData;
		check(Struct->EditorData);

		Struct->Guid = FGuid::NewGuid();
		if (Args.bIsBlueprintType)
		{
			UDSEditorData->MetaData.Add(FBlueprintTags::BlueprintType, TEXT("true"));
		}
		if (!Args.bAddPropertyIDToPropertyName)
		{
			UDSEditorData->MetaData.Add(NAME_GeneratePropertyWithoutID, TEXT("true"));
		}
		TMap<FName, FString>& Map = Struct->GetPackage()->GetMetaData().ObjectMetaDataMap.FindOrAdd(FSoftObjectPath(Struct));
		Map.Append(UDSEditorData->MetaData);

		Struct->Bind();
		constexpr bool bRelinkExistingProperties = true;
		Struct->StaticLink(bRelinkExistingProperties);
		UDSEditorData->RecreateDefaultInstance();

		Struct->Status = UDSS_UpToDate;
		return Struct;
	}

	UUserDefinedStruct* FindUserDefinedStruct(TNotNull<UObject*> InOuter, FName Name)
	{
		return Cast<UUserDefinedStruct>(StaticFindObject(UUserDefinedStruct::StaticClass(), InOuter, Name.ToString()));
	}

	bool Equal(const FStructVariableDescription& A, const FStructVariableDescription& B)
	{
		return A.HasSameType(B)
			&& A.VarName == B.VarName
			&& A.VarGuid == B.VarGuid
			&& A.DefaultValue == B.DefaultValue;
	}

	TValueOrError<FStructVariableDescription, void> CreateUserDefinedVariableFromDesc(const FPropertyBagPropertyDesc& PropertyDesc, bool bNameWithID)
	{
		if (!UE::StructUtils::CanCreateValidGraphPinTypeForPropertyDesc(PropertyDesc))
		{
			return MakeError();
		}

		const FEdGraphPinType GraphPin = UE::StructUtils::GetPropertyDescAsPin(PropertyDesc);

		auto IsAllowableBlueprintVariableType = [](const UObject* Object)
			{
				if (Object)
				{
					constexpr bool bForInternalUse = false;
					if (const UClass* Class = Cast<const UClass>(Object))
					{
						return UEdGraphSchema_K2::IsAllowableBlueprintVariableType(Class, bForInternalUse);
					}
					else if (const UScriptStruct* ScriptStruct = Cast<const UScriptStruct>(Object))
					{
						return UEdGraphSchema_K2::IsAllowableBlueprintVariableType(ScriptStruct, bForInternalUse);
					}
					else if (const UEnum* Enum = Cast<const UEnum>(Object))
					{
						return UEdGraphSchema_K2::IsAllowableBlueprintVariableType(Enum);
					}
				}
				return true;
			};

		// Test the struct/object/enum type.
		if (!IsAllowableBlueprintVariableType(GraphPin.PinSubCategoryObject.Get()))
		{
			return MakeError();
		}

		// Test the value (struct/object/enum) type.for map.
		if (!IsAllowableBlueprintVariableType(GraphPin.PinValueType.TerminalSubCategoryObject.Get()))
		{
			return MakeError();
		}

		FStructVariableDescription Result;
		Result.SetPinType(GraphPin);
		Result.FriendlyName = PropertyDesc.Name.ToString();
		Result.VarName = PropertyDesc.Name;
		if (bNameWithID)
		{
			// FMemberVariableNameHelper adds the ID to the name.
			Result.VarName = *FString::Printf(TEXT("%s_%s"), *Result.FriendlyName, *PropertyDesc.ID.ToString(EGuidFormats::Digits));
		}
		Result.VarGuid = PropertyDesc.ID;

		return MakeValue(MoveTemp(Result));
	}

	TValueOrError<FStructVariableDescription, void> CreateUserDefinedVariableFromDesc(const FInstancedPropertyBag& InstancedPropertyBag, const FPropertyBagPropertyDesc& PropertyDesc, bool bNameWithID)
	{
		TValueOrError<FStructVariableDescription, void> Result = CreateUserDefinedVariableFromDesc(PropertyDesc, bNameWithID);
		if (Result.HasError())
		{
			return MakeError();
		}

		const UScriptStruct* PropertyBag = InstancedPropertyBag.GetValue().GetScriptStruct();
		if (const FProperty* Property = PropertyBag->FindPropertyByName(PropertyDesc.Name))
		{
			UObject* OwningObject = const_cast<UScriptStruct*>(PropertyBag);
			const bool bValueAsString = FBlueprintEditorUtils::PropertyValueToString(Property, InstancedPropertyBag.GetValue().GetMemory(), Result.GetValue().DefaultValue, OwningObject, PPF_DuplicateVerbatim);
			ensure(bValueAsString);
		}
		return Result;
	}

	TValueOrError<TArray<FStructVariableDescription>, void> GetPropertyDescs(const FInstancedPropertyBag& PropertyBag, const TArrayView<const FPropertyBagPropertyDesc>  PropertyDescs, bool bNameWithID)
	{
		TArray<FStructVariableDescription> VariablesDescriptions;
		VariablesDescriptions.Reserve(PropertyDescs.Num());
		TArray<FPropertyBagPropertyDesc> SanitizedPropertyDescs = UPropertyBag::SanitizePropertyDescs(PropertyDescs);
		for (const FPropertyBagPropertyDesc& Desc : SanitizedPropertyDescs)
		{
			TValueOrError<FStructVariableDescription, void> Variable = CreateUserDefinedVariableFromDesc(PropertyBag, Desc, bNameWithID);
			if (Variable.HasError())
			{
				return MakeError();
			}
			VariablesDescriptions.Add(Variable.StealValue());
		}
		return MakeValue(MoveTemp(VariablesDescriptions));
	}
}

namespace UE::StructUtils
{
	TValueOrError<UUserDefinedStruct*, void> CreateUserDefinedStructFromDescs(TNotNull<UObject*> UDSPackage, const FInstancedPropertyBag& PropertyBag, FName UserDefinedStructName, const FCreateUserDefinedStructArgs& Args)
	{
		if (Private::FindUserDefinedStruct(UDSPackage, UserDefinedStructName))
		{
			return MakeError();
		}

		TNotNull<UUserDefinedStruct*> UserDefinedStruct = Private::CreateUserDefinedStruct(UDSPackage, UserDefinedStructName, Args);
		if (!PropertyBag.IsValid())
		{
			return MakeValue(UserDefinedStruct);
		}

		TArrayView<const FPropertyBagPropertyDesc> PropertyDescs = PropertyBag.GetPropertyBagStruct()->GetPropertyDescs();
		if (PropertyDescs.IsEmpty())
		{
			return MakeValue(UserDefinedStruct);
		}

		TValueOrError<TArray<FStructVariableDescription>, void> VariableDescriptionsResult = Private::GetPropertyDescs(PropertyBag, PropertyDescs, Args.bAddPropertyIDToPropertyName);
		if (VariableDescriptionsResult.HasError())
		{
			return MakeError();
		}

		TNotNull<UUserDefinedStructEditorData*> EditorData = CastChecked<UUserDefinedStructEditorData>(UserDefinedStruct->EditorData);
		EditorData->VariablesDescriptions = VariableDescriptionsResult.StealValue();
		FStructureEditorUtils::CompileStructure(UserDefinedStruct);

		return MakeValue(UserDefinedStruct);
	}

	TValueOrError<bool, void> UpdateUserDefinedStructFromDescs(TNotNull<UUserDefinedStruct*> UserDefinedStruct, const FInstancedPropertyBag& PropertyBag)
	{
		bool bEqual = true;
		UUserDefinedStructEditorData* EditorData = Cast<UUserDefinedStructEditorData>(UserDefinedStruct->EditorData);
		if (ensure(EditorData))
		{
			TArrayView<const FPropertyBagPropertyDesc> PropertyDescs;
			if (PropertyBag.IsValid())
			{
				PropertyDescs = PropertyBag.GetPropertyBagStruct()->GetPropertyDescs();
			}

			const FString* FoundGeneratePropertyWithoutID = EditorData->MetaData.Find(Private::NAME_GeneratePropertyWithoutID);
			const bool bNameWithoutID = FoundGeneratePropertyWithoutID ? *FoundGeneratePropertyWithoutID == TEXT("true") : false;
			TValueOrError<TArray<FStructVariableDescription>, void> VariableDescriptionsResult = Private::GetPropertyDescs(PropertyBag, PropertyDescs, !bNameWithoutID);
			if (VariableDescriptionsResult.HasError())
			{
				return MakeError();
			}

			TArray<FStructVariableDescription>& VariablesDescriptions = VariableDescriptionsResult.GetValue();
			bEqual = EditorData->VariablesDescriptions.Num() == VariablesDescriptions.Num();
			if (bEqual)
			{
				for (int32 Index = 0; Index < VariablesDescriptions.Num(); ++Index)
				{
					if (!Private::Equal(VariablesDescriptions[Index], EditorData->VariablesDescriptions[Index]))
					{
						bEqual = false;
						break;
					}
				}
			}

			if (!bEqual)
			{
				// Update the Current Value before setting the Default value.
				if (const uint8* StructData = UserDefinedStruct->GetDefaultInstance())
				{
					for (FStructVariableDescription& Description : VariablesDescriptions)
					{
						if (FProperty* Property = EditorData->FindProperty(UserDefinedStruct, Description.VarName))
						{
							FBlueprintEditorUtils::PropertyValueToString(Property, StructData, Description.CurrentDefaultValue, UserDefinedStruct);
						}
					}
				}
				EditorData->VariablesDescriptions = MoveTemp(VariablesDescriptions);
				FStructureEditorUtils::CompileStructure(UserDefinedStruct);
			}
		}

		return MakeValue(!bEqual);
	}
}

namespace UE::StructUtils::Private
{
	TOptional<FFindUserFunctionResult> FindUserFunction(const TSharedPtr<IPropertyHandle>& InStructProperty, FName InFuncMetadataName)
	{
		FProperty* MetadataProperty = InStructProperty->GetMetaDataProperty();

		FFindUserFunctionResult Result{};

		if (!MetadataProperty || !MetadataProperty->HasMetaData(InFuncMetadataName))
		{
			return {};
		}

		FString FunctionName = MetadataProperty->GetMetaData(InFuncMetadataName);
		if (FunctionName.IsEmpty())
		{
			return {};
		}

		TArray<UObject*> OutObjects;
		InStructProperty->GetOuterObjects(OutObjects);

		// Check for external function references, taken from GetOptions
		if (FunctionName.Contains(TEXT(".")))
		{
			Result.Function = FindObject<UFunction>(nullptr, *FunctionName, EFindObjectFlags::ExactClass);

			if (ensureMsgf(Result.Function && Result.Function->HasAnyFunctionFlags(EFunctionFlags::FUNC_Static), TEXT("[%s] Didn't find function %s or expected it to be static"), *InFuncMetadataName.ToString(), *FunctionName))
			{
				UObject* FunctionCDO = Result.Function->GetOuterUClass()->GetDefaultObject();
				Result.Target = FunctionCDO;
			}
			else
			{
				return {};
			}
		}
		else if (OutObjects.Num() > 0)
		{
			Result.Target = OutObjects[0];
			Result.Function = Result.Target->GetClass() ? Result.Target->GetClass()->FindFunctionByName(*FunctionName) : nullptr;
		}

		// Only support native functions
		if (!ensureMsgf(Result.Function && Result.Function->IsNative(), TEXT("[%s] Didn't find function %s or expected it to be native"), *InFuncMetadataName.ToString(), *FunctionName))
		{
			return {};
		}

		return (Result.Target != nullptr && Result.Function != nullptr) ? Result : TOptional<FFindUserFunctionResult>{};
	}
} // UE::StructUtils::Private
