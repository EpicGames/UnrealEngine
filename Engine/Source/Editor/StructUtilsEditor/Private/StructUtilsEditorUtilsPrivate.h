// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"

class UFunction;
class UObject;

namespace UE::StructUtils::Private
{
	
struct FFindUserFunctionResult
{
	UFunction* Function = nullptr;
	UObject* Target = nullptr;
};

/** @return the member and the member function that exist on the property handle object. */
TOptional<FFindUserFunctionResult> FindUserFunction(const TSharedPtr<IPropertyHandle>& InStructProperty, FName InFuncMetadataName);

/** @return true property handle holds struct property of type T.  */
template<typename T> requires TModels_V<CStaticStructProvider, T>
bool IsScriptStruct(const TSharedPtr<IPropertyHandle>& PropertyHandle)
{
	if (!PropertyHandle)
	{
		return false;
	}

	//@TODO we want IsChild and not IsA
	const FStructProperty* StructProperty = CastField<const FStructProperty>(PropertyHandle->GetProperty());
	return StructProperty && StructProperty->Struct->IsA(TBaseStructure<T>::Get()->GetClass());
}

} // UE::StructUtils::Private
