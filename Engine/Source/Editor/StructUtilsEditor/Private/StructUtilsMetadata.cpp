// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtilsMetadata.h"

#include "StructUtils/PropertyBag.h"

namespace UE::StructUtils::Metadata
{
	// For properties
	const FLazyName EnableCategoriesName = "EnableCategories";
	const FLazyName CategoryName = "Category";

	// For the bag
	const FLazyName HideInDetailPanelsName = "HideInDetailPanel";
	const FLazyName ShowOnlyInnerPropertiesName = "ShowOnlyInnerProperties";
	const FLazyName FixedLayoutName = "FixedLayout";
	const FLazyName DefaultTypeName = "DefaultType";
	const FLazyName AllowContainersName = "AllowContainers";
	const FLazyName PropertyBagSchemaClassName = "PropertyBagSchemaClass";
	UE_DEPRECATED(5.6, "Use 'AllowContainers' instead.");
	const FLazyName AllowArraysName = "AllowArrays";
	const FLazyName IsPinTypeAcceptedName = "IsPinTypeAccepted";
	const FLazyName CanRemovePropertyName = "CanRemoveProperty";
	const FLazyName ChildRowFeaturesName = "ChildRowFeatures";
	const FLazyName ExtensionClass = "ExtensionClass";

	const FLazyName BaseStructName = "BaseStruct";
	const FLazyName BaseStructFunctionName = "BaseStructFunction";
	const FLazyName StructTypeConstName = "StructTypeConst";
	const FLazyName HideStructTypeName = "HideStructType";

	bool AreCategoriesEnabled(const FPropertyBagPropertyDesc& Desc)
	{
		return Desc.HasMetaData(EnableCategoriesName);
	}

	void EnableCategories(FPropertyBagPropertyDesc& OutDesc)
	{
		if (!AreCategoriesEnabled(OutDesc))
		{
			OutDesc.MetaData.Emplace(FPropertyBagPropertyDescMetaData(EnableCategoriesName, {}));
		}
	}

	void DisableCategories(FPropertyBagPropertyDesc& OutDesc)
	{
		OutDesc.MetaData.RemoveAllSwap([](const FPropertyBagPropertyDescMetaData& MetaData)
		{
			return MetaData.Key == EnableCategoriesName;
		});
	}

	void SetCategory(FPropertyBagPropertyDesc& OutDesc, const FString& InGroupLabel, const bool bAutoEnableCategories)
	{
		OutDesc.SetMetaData(CategoryName, InGroupLabel);

		if (bAutoEnableCategories)
		{
			EnableCategories(OutDesc);
		}
	}

	void RemoveCategory(FPropertyBagPropertyDesc& OutDesc, const bool bAutoDisableCategories)
	{
		OutDesc.MetaData.RemoveAllSwap([](const FPropertyBagPropertyDescMetaData& MetaData)
		{
			return MetaData.Key == CategoryName;
		});

		if (bAutoDisableCategories)
		{
			DisableCategories(OutDesc);
		}
	}

	FString GetCategory(const FPropertyBagPropertyDesc& InDesc)
	{
		return InDesc.GetMetaData(CategoryName);
	}

	namespace Specifiers
	{
		const FLazyName ToolTipName = "ToolTip";
		const FLazyName ClampMinName = "ClampMin";
		const FLazyName ClampMaxName = "ClampMax";
		const FLazyName UIMinName = "UIMin";
		const FLazyName UIMaxName = "UIMax";
		const FLazyName UnitsName = "Units";
		const FLazyName ForceUnitsName = "ForceUnits";

		bool SupportsClamping(const FPropertyBagPropertyDesc& InDesc)
		{
			return InDesc.IsNumericType();
		}

		bool SupportsSettingUnits(const FPropertyBagPropertyDesc& InDesc)
		{
			return InDesc.IsNumericType();
		}
	}
}
