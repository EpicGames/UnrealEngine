// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyBagDiff.h"

#include "Misc/AutomationTest.h"
#include "StructUtils/PropertyBag.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::StructUtils::Tests
{
	namespace Private
	{
		/** Build an FInstancedPropertyBag from a descriptor array with explicit, caller-controlled IDs. */
		static FInstancedPropertyBag MakeBag(TArray<FPropertyBagPropertyDesc> Descs)
		{
			const UPropertyBag* BagStruct = UPropertyBag::GetOrCreateFromDescs(Descs);
			FInstancedPropertyBag Bag;
			Bag.InitializeFromBagStruct(BagStruct);
			return Bag;
		}

		/** Helper that builds a descriptor with explicit ID + name + type + flags + (optional) metadata. */
		static FPropertyBagPropertyDesc MakeDesc(const FGuid& ID, FName Name, EPropertyBagPropertyType Type, EPropertyFlags Flags = CPF_Edit, TArray<FPropertyBagPropertyDescMetaData> MetaData = {})
		{
			FPropertyBagPropertyDesc Desc(Name, Type);
			Desc.ID = ID;
			Desc.PropertyFlags = static_cast<uint64>(Flags);
#if WITH_EDITORONLY_DATA
			Desc.MetaData = MoveTemp(MetaData);
#endif
			return Desc;
		}

		/** Find the first diff matching a particular ID/type. Returns nullptr if absent. */
		static const FPropertyBagDiffEntry* FindDiff(const TArray<FPropertyBagDiffEntry>& Diffs, const FGuid& ID, EPropertyBagDiffType Type)
		{
			for (const FPropertyBagDiffEntry& Entry : Diffs)
			{
				if (Entry.PropertyID == ID && Entry.DiffType == Type)
				{
					return &Entry;
				}
			}
			return nullptr;
		}

		static int32 CountDiffsOfType(const TArray<FPropertyBagDiffEntry>& Diffs, EPropertyBagDiffType Type)
		{
			int32 Count = 0;
			for (const FPropertyBagDiffEntry& Entry : Diffs)
			{
				if (Entry.DiffType == Type)
				{
					++Count;
				}
			}
			return Count;
		}
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_PropertyBagDiff_AddedRemoved
		, "System.StructUtils.PropertyBagDiff.AddedRemoved"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_PropertyBagDiff_AddedRemoved::RunTest(const FString& Parameters)
	{
		using namespace Private;

		const FGuid IDKept = FGuid::NewGuid();
		const FGuid IDRemoved = FGuid::NewGuid();
		const FGuid IDAdded = FGuid::NewGuid();

		FInstancedPropertyBag Old = MakeBag({
			MakeDesc(IDKept,    "Kept",    EPropertyBagPropertyType::Int32),
			MakeDesc(IDRemoved, "Removed", EPropertyBagPropertyType::Float),
		});

		FInstancedPropertyBag New = MakeBag({
			MakeDesc(IDKept,  "Kept",  EPropertyBagPropertyType::Int32),
			MakeDesc(IDAdded, "Added", EPropertyBagPropertyType::Bool),
		});

		TArray<FPropertyBagDiffEntry> Diffs;
		DiffPropertyBags(Old, New, Diffs);

		UTEST_EQUAL("Two diff entries (one Added + one Removed)", Diffs.Num(), 2);
		UTEST_NOT_NULL("Removed entry present", FindDiff(Diffs, IDRemoved, EPropertyBagDiffType::Removed));
		UTEST_NOT_NULL("Added entry present", FindDiff(Diffs, IDAdded, EPropertyBagDiffType::Added));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_PropertyBagDiff_Rename
		, "System.StructUtils.PropertyBagDiff.Rename"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_PropertyBagDiff_Rename::RunTest(const FString& Parameters)
	{
		using namespace Private;

		const FGuid ID = FGuid::NewGuid();

		FInstancedPropertyBag Old = MakeBag({ MakeDesc(ID, "OldName", EPropertyBagPropertyType::Int32) });
		FInstancedPropertyBag New = MakeBag({ MakeDesc(ID, "NewName", EPropertyBagPropertyType::Int32) });

		TArray<FPropertyBagDiffEntry> Diffs;
		DiffPropertyBags(Old, New, Diffs);

		UTEST_EQUAL("Exactly one diff entry", Diffs.Num(), 1);
		const FPropertyBagDiffEntry* RenameEntry = FindDiff(Diffs, ID, EPropertyBagDiffType::Renamed);
		UTEST_NOT_NULL("Renamed entry present", RenameEntry);
		if (RenameEntry)
		{
			UTEST_EQUAL("Renamed OldName", RenameEntry->OldName, FName("OldName"));
			UTEST_EQUAL("Renamed NewName", RenameEntry->NewName, FName("NewName"));
		}
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_PropertyBagDiff_TypeChanged
		, "System.StructUtils.PropertyBagDiff.TypeChanged"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_PropertyBagDiff_TypeChanged::RunTest(const FString& Parameters)
	{
		using namespace Private;

		const FGuid ID = FGuid::NewGuid();

		FInstancedPropertyBag Old = MakeBag({ MakeDesc(ID, "P", EPropertyBagPropertyType::Int32) });
		FInstancedPropertyBag New = MakeBag({ MakeDesc(ID, "P", EPropertyBagPropertyType::Float) });

		TArray<FPropertyBagDiffEntry> Diffs;
		DiffPropertyBags(Old, New, Diffs);

		UTEST_EQUAL("Exactly one diff entry", Diffs.Num(), 1);
		UTEST_NOT_NULL("TypeChanged entry present", FindDiff(Diffs, ID, EPropertyBagDiffType::TypeChanged));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_PropertyBagDiff_ValueChanged
		, "System.StructUtils.PropertyBagDiff.ValueChanged"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_PropertyBagDiff_ValueChanged::RunTest(const FString& Parameters)
	{
		using namespace Private;

		const FGuid IDA = FGuid::NewGuid();
		const FGuid IDB = FGuid::NewGuid();

		FInstancedPropertyBag Old = MakeBag({
			MakeDesc(IDA, "A", EPropertyBagPropertyType::Int32),
			MakeDesc(IDB, "B", EPropertyBagPropertyType::Float),
		});
		Old.SetValueInt32("A", 1);
		Old.SetValueFloat("B", 1.0f);

		FInstancedPropertyBag New = MakeBag({
			MakeDesc(IDA, "A", EPropertyBagPropertyType::Int32),
			MakeDesc(IDB, "B", EPropertyBagPropertyType::Float),
		});
		New.SetValueInt32("A", 1);    // same
		New.SetValueFloat("B", 99.f); // changed

		TArray<FPropertyBagDiffEntry> Diffs;
		DiffPropertyBags(Old, New, Diffs);

		UTEST_EQUAL("Exactly one diff entry", Diffs.Num(), 1);
		UTEST_NOT_NULL("ValueChanged on B", FindDiff(Diffs, IDB, EPropertyBagDiffType::ValueChanged));
		return true;
	}

#if WITH_EDITORONLY_DATA
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_PropertyBagDiff_CategoryChanged
		, "System.StructUtils.PropertyBagDiff.CategoryChanged"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_PropertyBagDiff_CategoryChanged::RunTest(const FString& Parameters)
	{
		using namespace Private;

		const FGuid ID = FGuid::NewGuid();

		FInstancedPropertyBag Old = MakeBag({ MakeDesc(ID, "P", EPropertyBagPropertyType::Int32, CPF_Edit, { { FName("Category"), TEXT("First") } }) });
		FInstancedPropertyBag New = MakeBag({ MakeDesc(ID, "P", EPropertyBagPropertyType::Int32, CPF_Edit, { { FName("Category"), TEXT("Second") } }) });

		TArray<FPropertyBagDiffEntry> Diffs;
		DiffPropertyBags(Old, New, Diffs);

		UTEST_NOT_NULL("CategoryChanged entry present", FindDiff(Diffs, ID, EPropertyBagDiffType::CategoryChanged));
		UTEST_EQUAL("No MetaDataChanged entry", CountDiffsOfType(Diffs, EPropertyBagDiffType::MetaDataChanged), 0);
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_PropertyBagDiff_MetaDataChanged
		, "System.StructUtils.PropertyBagDiff.MetaDataChanged"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_PropertyBagDiff_MetaDataChanged::RunTest(const FString& Parameters)
	{
		using namespace Private;

		const FGuid ID = FGuid::NewGuid();

		FInstancedPropertyBag Old = MakeBag({ MakeDesc(ID, "P", EPropertyBagPropertyType::Int32, CPF_Edit, { { FName("Tooltip"), TEXT("Hello") } }) });
		FInstancedPropertyBag New = MakeBag({ MakeDesc(ID, "P", EPropertyBagPropertyType::Int32, CPF_Edit, { { FName("Tooltip"), TEXT("World") } }) });

		TArray<FPropertyBagDiffEntry> Diffs;
		DiffPropertyBags(Old, New, Diffs);

		const FPropertyBagDiffEntry* MetaEntry = FindDiff(Diffs, ID, EPropertyBagDiffType::MetaDataChanged);
		UTEST_NOT_NULL("MetaDataChanged entry present", MetaEntry);
		if (MetaEntry)
		{
			UTEST_EQUAL("MetaDataKey is Tooltip", MetaEntry->MetaDataKey, FName("Tooltip"));
		}
		UTEST_EQUAL("No CategoryChanged entry", CountDiffsOfType(Diffs, EPropertyBagDiffType::CategoryChanged), 0);
		return true;
	}
#endif // WITH_EDITORONLY_DATA

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_PropertyBagDiff_FlagsChanged
		, "System.StructUtils.PropertyBagDiff.FlagsChanged"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_PropertyBagDiff_FlagsChanged::RunTest(const FString& Parameters)
	{
		using namespace Private;

		const FGuid ID = FGuid::NewGuid();

		FInstancedPropertyBag Old = MakeBag({ MakeDesc(ID, "P", EPropertyBagPropertyType::Int32, CPF_Edit) });
		FInstancedPropertyBag New = MakeBag({ MakeDesc(ID, "P", EPropertyBagPropertyType::Int32,
			static_cast<EPropertyFlags>(CPF_Edit | CPF_BlueprintReadOnly)) });

		TArray<FPropertyBagDiffEntry> Diffs;
		DiffPropertyBags(Old, New, Diffs);

		UTEST_NOT_NULL("FlagsChanged entry present", FindDiff(Diffs, ID, EPropertyBagDiffType::FlagsChanged));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_PropertyBagDiff_Reordered
		, "System.StructUtils.PropertyBagDiff.Reordered"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_PropertyBagDiff_Reordered::RunTest(const FString& Parameters)
	{
		using namespace Private;

		const FGuid IDA = FGuid::NewGuid();
		const FGuid IDB = FGuid::NewGuid();

		FInstancedPropertyBag Old = MakeBag({
			MakeDesc(IDA, "A", EPropertyBagPropertyType::Int32),
			MakeDesc(IDB, "B", EPropertyBagPropertyType::Int32),
		});
		FInstancedPropertyBag New = MakeBag({
			MakeDesc(IDB, "B", EPropertyBagPropertyType::Int32),
			MakeDesc(IDA, "A", EPropertyBagPropertyType::Int32),
		});

		TArray<FPropertyBagDiffEntry> Diffs;
		DiffPropertyBags(Old, New, Diffs);

		// Both surviving properties move position when swapped.
		UTEST_EQUAL("Two Reordered entries", CountDiffsOfType(Diffs, EPropertyBagDiffType::Reordered), 2);
		UTEST_NOT_NULL("Reordered A", FindDiff(Diffs, IDA, EPropertyBagDiffType::Reordered));
		UTEST_NOT_NULL("Reordered B", FindDiff(Diffs, IDB, EPropertyBagDiffType::Reordered));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_PropertyBagDiff_CombinedChanges
		, "System.StructUtils.PropertyBagDiff.CombinedChanges"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_PropertyBagDiff_CombinedChanges::RunTest(const FString& Parameters)
	{
		using namespace Private;

		const FGuid ID = FGuid::NewGuid();

		FInstancedPropertyBag Old = MakeBag({ MakeDesc(ID, "OldName", EPropertyBagPropertyType::Int32, CPF_Edit, { { FName("Category"), TEXT("First") } }) });
		Old.SetValueInt32("OldName", 1);

		FInstancedPropertyBag New = MakeBag({ MakeDesc(ID, "NewName", EPropertyBagPropertyType::Int32, CPF_Edit, { { FName("Category"), TEXT("Second") } }) });
		New.SetValueInt32("NewName", 42);

		TArray<FPropertyBagDiffEntry> Diffs;
		DiffPropertyBags(Old, New, Diffs);

		UTEST_NOT_NULL("Renamed entry", FindDiff(Diffs, ID, EPropertyBagDiffType::Renamed));
#if WITH_EDITORONLY_DATA
		UTEST_NOT_NULL("CategoryChanged entry", FindDiff(Diffs, ID, EPropertyBagDiffType::CategoryChanged));
#endif
		UTEST_NOT_NULL("ValueChanged entry", FindDiff(Diffs, ID, EPropertyBagDiffType::ValueChanged));
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_PropertyBagDiff_EmptyAndIdentical
		, "System.StructUtils.PropertyBagDiff.EmptyAndIdentical"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_PropertyBagDiff_EmptyAndIdentical::RunTest(const FString& Parameters)
	{
		using namespace Private;

		// Empty vs empty: no diffs.
		{
			FInstancedPropertyBag Empty1;
			FInstancedPropertyBag Empty2;
			TArray<FPropertyBagDiffEntry> Diffs;
			DiffPropertyBags(Empty1, Empty2, Diffs);
			UTEST_EQUAL("Empty vs empty produces no diffs", Diffs.Num(), 0);
		}

		// Same bag vs itself (copy): no diffs.
		{
			const FGuid ID = FGuid::NewGuid();
			FInstancedPropertyBag Bag = MakeBag({ MakeDesc(ID, "P", EPropertyBagPropertyType::Int32) });
			Bag.SetValueInt32("P", 7);
			FInstancedPropertyBag Copy = Bag;
			TArray<FPropertyBagDiffEntry> Diffs;
			DiffPropertyBags(Bag, Copy, Diffs);
			UTEST_EQUAL("Identical bag vs copy produces no diffs", Diffs.Num(), 0);
		}

		// Empty vs non-empty: every new property is Added.
		{
			const FGuid IDA = FGuid::NewGuid();
			const FGuid IDB = FGuid::NewGuid();
			FInstancedPropertyBag Empty;
			FInstancedPropertyBag NonEmpty = MakeBag({
				MakeDesc(IDA, "A", EPropertyBagPropertyType::Int32),
				MakeDesc(IDB, "B", EPropertyBagPropertyType::Float),
			});
			TArray<FPropertyBagDiffEntry> Diffs;
			DiffPropertyBags(Empty, NonEmpty, Diffs);
			UTEST_EQUAL("Empty vs non-empty: every property is Added", CountDiffsOfType(Diffs, EPropertyBagDiffType::Added), 2);
		}

		return true;
	}
}

#endif // WITH_DEV_AUTOMATION_TESTS
