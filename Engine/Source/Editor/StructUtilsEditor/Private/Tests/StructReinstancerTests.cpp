// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/StructReinstancerTests.h"

#include "Misc/AutomationTest.h"

#include "StructUtils/StructReinstancer.h"
#include "StructUtilsEditorUtils.h"

#define LOCTEXT_NAMESPACE "StructReinstancerTests"

namespace UE::StructUtils::Private
{
	template<typename T>
	T GetPropertyValue_InContainer(TNotNull<const FProperty*> Property, TNotNull<const uint8*> Container)
	{
		T LocalValue{};
		Property->GetValue_InContainer(Container, &LocalValue);
		return LocalValue;
	}

	template<typename T>
	TOptional<T> TryGetUDSPropertyValue_InContainer(TNotNull<const UUserDefinedStruct*> UserDefinedStruct, FName PropertyName, const uint8* Container)
	{
		if (Container == nullptr)
		{
			return {};
		}

		const UUserDefinedStructEditorData* EditorData = Cast<UUserDefinedStructEditorData>(UserDefinedStruct->EditorData);
		if (EditorData == nullptr)
		{
			return {};
		}

		const FProperty* Property = EditorData->FindProperty(UserDefinedStruct, PropertyName);
		if (Property == nullptr)
		{
			return {};
		}

		T LocalValue{};
		Property->GetValue_InContainer(Container, &LocalValue);
		return LocalValue;
	}

	template<typename T>
	TOptional<T> TryGetPropertyValue_InContainer(TNotNull<const UScriptStruct*> ScriptStruct, FName PropertyName, const uint8* Container)
	{
		if (Container == nullptr)
		{
			return {};
		}

		const FProperty* Property = ScriptStruct->FindPropertyByName(PropertyName);
		if (Property == nullptr)
		{
			return {};
		}

		T LocalValue{};
		Property->GetValue_InContainer(Container, &LocalValue);
		return LocalValue;
	}

	template<typename T>
	bool TrySetUDSPropertyValue_InContainer(TNotNull<UUserDefinedStruct*> UserDefinedStruct, FName PropertyName, uint8* Container, T Value)
	{
		if (Container == nullptr)
		{
			return false;
		}

		const UUserDefinedStructEditorData* EditorData = Cast<UUserDefinedStructEditorData>(UserDefinedStruct->EditorData);
		if (EditorData == nullptr)
		{
			return false;
		}

		const FProperty* Property = EditorData->FindProperty(UserDefinedStruct, PropertyName);
		if (Property == nullptr)
		{
			return false;
		}

		Property->SetValue_InContainer(Container, &Value);
		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_UserDefinedStructReinstancer
		, "System.StructUtils.Reinstancer.UserDefinedStruct"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_UserDefinedStructReinstancer::RunTest(const FString& Parameters)
	{
		static const FName AName("A");
		static const FName BName("B");
		static const FName CName("C");

		TStrongObjectPtr<UTestObjectThatContainsStruct> UDSContainer(NewObject<UTestObjectThatContainsStruct>());

		// Create UDS
		FInstancedPropertyBag BagA;
		BagA.AddProperties({
			{ AName, EPropertyBagPropertyType::String }, // add a string to check if the stomp allocator might find something
			{ BName, EPropertyBagPropertyType::Int32 }
			});
		BagA.SetValueString(AName, TEXT("A"));
		BagA.SetValueInt32(BName, 2);

		UUserDefinedStruct* UserDefinedStruct = nullptr;
		{
			UE::StructUtils::FCreateUserDefinedStructArgs Args;
			Args.bAddPropertyIDToPropertyName = true;
			Args.bIsBlueprintType = false;
			TValueOrError<UUserDefinedStruct*, void> Result = UE::StructUtils::CreateUserDefinedStructFromDescs(UDSContainer.Get(), BagA, "UDS", Args);

			UTEST_TRUE("UUserDefinedStruct created", Result.HasValue());
			UserDefinedStruct = Result.GetValue();
			UTEST_TRUE("UUserDefinedStruct valid", UserDefinedStruct != nullptr);
		}

		// Assign to test object
		TStrongObjectPtr<UTestObjectThatContainsStruct> TestInstancedObject1(NewObject<UTestObjectThatContainsStruct>());
		TestInstancedObject1->InstancedStruct = UserDefinedStruct;
		TStrongObjectPtr<UTestObjectThatContainsStruct> TestInstancedObject2(NewObject<UTestObjectThatContainsStruct>());
		TestInstancedObject2->Container.InstancedStruct = UserDefinedStruct;
		TStrongObjectPtr<UTestObjectThatContainsStruct> TestContainerObject1(NewObject<UTestObjectThatContainsStruct>());
		TestContainerObject1->InstancedStructContainer.InsertAt(0, TArrayView<FInstancedStruct>(&TestInstancedObject1->InstancedStruct, 1));
		TStrongObjectPtr<UTestObjectThatContainsStruct> TestContainerObject2(NewObject<UTestObjectThatContainsStruct>());
		TestContainerObject2->Container.InstancedStructContainer.InsertAt(0, TArrayView<FInstancedStruct>(&TestInstancedObject1->InstancedStruct, 1));
		TStrongObjectPtr<UTestObjectThatContainsStruct> TestBagObject1(NewObject<UTestObjectThatContainsStruct>());
		TestBagObject1->InstancedPropertyBag.AddProperties({{"MyStruct1", EPropertyBagPropertyType::Struct, UserDefinedStruct}});
		TStrongObjectPtr<UTestObjectThatContainsStruct> TestBagObject2(NewObject<UTestObjectThatContainsStruct>());
		TestBagObject2->Container.InstancedPropertyBag.AddProperties({ {"MyStruct2", EPropertyBagPropertyType::Struct, UserDefinedStruct} });

		{
			UTEST_TRUE("TestInstancedObject1.InstancedStruct struct", TestInstancedObject1->InstancedStruct.GetScriptStruct() == UserDefinedStruct);
			UTEST_TRUE("TestInstancedObject1.Container.InstancedStruct struct", TestInstancedObject1->Container.InstancedStruct.GetScriptStruct() == nullptr);
			UTEST_TRUE("TestInstancedObject2.InstancedStruct struct", TestInstancedObject2->Container.InstancedStruct.GetScriptStruct() == UserDefinedStruct);
			UTEST_TRUE("TestInstancedObject2.Container.InstancedStruct struct", TestInstancedObject2->InstancedStruct.GetScriptStruct() == nullptr);

			UTEST_TRUE("TestContainerObject1.InstancedStructContainer struct", TestContainerObject1->InstancedStructContainer[0].GetScriptStruct() == UserDefinedStruct);
			UTEST_TRUE("TestContainerObject1.Container.InstancedStructContainer struct", TestContainerObject1->Container.InstancedStructContainer.Num() == 0);
			UTEST_TRUE("TestContainerObject2.InstancedStructContainer struct", TestContainerObject2->Container.InstancedStructContainer[0].GetScriptStruct() == UserDefinedStruct);
			UTEST_TRUE("TestContainerObject2.Container.InstancedStructContainer struct", TestContainerObject2->InstancedStructContainer.Num() == 0);

			UTEST_TRUE("TestBagObject1.InstancedPropertyBag struct", TestBagObject1->InstancedPropertyBag.GetValue().GetScriptStruct() != UserDefinedStruct);
			UTEST_TRUE("TestBagObject1.Container.InstancedPropertyBag struct", TestBagObject1->Container.InstancedPropertyBag.GetValue().GetScriptStruct() == nullptr);
			UTEST_TRUE("TestBagObject2.InstancedPropertyBag struct", TestBagObject2->Container.InstancedPropertyBag.GetValue().GetScriptStruct() != UserDefinedStruct);
			UTEST_TRUE("TestBagObject2.Container.InstancedPropertyBag struct", TestBagObject2->InstancedPropertyBag.GetValue().GetScriptStruct() == nullptr);

			UTEST_TRUE("TestInstancedObject1 B Value", 2 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, TestInstancedObject1->InstancedStruct.GetMemory()).Get(0));
			UTEST_TRUE("TestInstancedObject1 B Value", !TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, TestInstancedObject1->Container.InstancedStruct.GetMemory()).IsSet());
			UTEST_TRUE("TestInstancedObject2 B Value", !TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, TestInstancedObject2->InstancedStruct.GetMemory()).IsSet());
			UTEST_TRUE("TestInstancedObject2 B Value", 2 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, TestInstancedObject2->Container.InstancedStruct.GetMemory()).Get(0));

			UTEST_TRUE("TestContainerObject1 B Value", 2 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, TestContainerObject1->InstancedStructContainer[0].GetMemory()).Get(0));
			UTEST_TRUE("TestContainerObject2 B Value", 2 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, TestContainerObject2->Container.InstancedStructContainer[0].GetMemory()).Get(0));

			TValueOrError<FStructView, EPropertyBagResult> BagResult1 = TestBagObject1->InstancedPropertyBag.GetValueStruct("MyStruct1");
			UTEST_TRUE(TEXT("BagResult1"), BagResult1.HasValue());
			UTEST_TRUE(TEXT("BagResult1 struct"), BagResult1.GetValue().GetScriptStruct() == UserDefinedStruct);
			UTEST_TRUE(TEXT("BagResult1 B Value"), 2 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, BagResult1.GetValue().GetMemory()).Get(0));
			TValueOrError<FStructView, EPropertyBagResult> BagResult2 = TestBagObject2->Container.InstancedPropertyBag.GetValueStruct("MyStruct2");
			UTEST_TRUE(TEXT("BagResult2"), BagResult2.HasValue());
			UTEST_TRUE(TEXT("BagResult2 struct"), BagResult2.GetValue().GetScriptStruct() == UserDefinedStruct);
			UTEST_TRUE(TEXT("BagResult2 B Value"), 2 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, BagResult2.GetValue().GetMemory()).Get(0));
		}

		// Add new property
		{
			BagA.AddProperties({ { CName, EPropertyBagPropertyType::Int32 } });
			BagA.SetValueInt32(BName, 22);
			BagA.SetValueInt32(CName, 3);

			const TValueOrError<bool, void> ModifyResult = UE::StructUtils::UpdateUserDefinedStructFromDescs(UserDefinedStruct, BagA);
			UTEST_TRUE("UUserDefinedStruct modified", ModifyResult.HasValue() && ModifyResult.GetValue());

			{
				UTEST_TRUE(TEXT("TestInstancedObject1.InstancedStruct struct"), TestInstancedObject1->InstancedStruct.GetScriptStruct() == UserDefinedStruct);
				UTEST_TRUE(TEXT("TestInstancedObject1.Container.InstancedStruct struct"), TestInstancedObject1->Container.InstancedStruct.GetScriptStruct() == nullptr);
				UTEST_TRUE(TEXT("TestInstancedObject2.InstancedStruct struct"), TestInstancedObject2->Container.InstancedStruct.GetScriptStruct() == UserDefinedStruct);
				UTEST_TRUE(TEXT("TestInstancedObject2.Container.InstancedStruct struct"), TestInstancedObject2->InstancedStruct.GetScriptStruct() == nullptr);

				UTEST_TRUE(TEXT("TestContainerObject1.InstancedStructContainer struct"), TestContainerObject1->InstancedStructContainer[0].GetScriptStruct() == UserDefinedStruct);
				UTEST_TRUE(TEXT("TestContainerObject1.Container.InstancedStructContainer struct"), TestContainerObject1->Container.InstancedStructContainer.Num() == 0);
				UTEST_TRUE(TEXT("TestContainerObject2.InstancedStructContainer struct"), TestContainerObject2->Container.InstancedStructContainer[0].GetScriptStruct() == UserDefinedStruct);
				UTEST_TRUE(TEXT("TestContainerObject2.Container.InstancedStructContainer struct"), TestContainerObject2->InstancedStructContainer.Num() == 0);

				UTEST_TRUE(TEXT("TestBagObject1.InstancedPropertyBag struct"), TestBagObject1->InstancedPropertyBag.GetValue().GetScriptStruct() != UserDefinedStruct);
				UTEST_TRUE(TEXT("TestBagObject1.Container.InstancedPropertyBag struct"), TestBagObject1->Container.InstancedPropertyBag.GetValue().GetScriptStruct() == nullptr);
				UTEST_TRUE(TEXT("TestBagObject2.InstancedPropertyBag struct"), TestBagObject2->Container.InstancedPropertyBag.GetValue().GetScriptStruct() != UserDefinedStruct);
				UTEST_TRUE(TEXT("TestBagObject2.Container.InstancedPropertyBag struct"), TestBagObject2->InstancedPropertyBag.GetValue().GetScriptStruct() == nullptr);

				UTEST_TRUE(TEXT("TestInstancedObject1 B Value"), 22 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, TestInstancedObject1->InstancedStruct.GetMemory()).Get(0));
				UTEST_TRUE(TEXT("TestInstancedObject1 B Value"), !TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, TestInstancedObject1->Container.InstancedStruct.GetMemory()).IsSet());
				UTEST_TRUE(TEXT("TestInstancedObject2 B Value"), !TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, TestInstancedObject2->InstancedStruct.GetMemory()).IsSet());
				UTEST_TRUE(TEXT("TestInstancedObject2 B Value"), 22 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, TestInstancedObject2->Container.InstancedStruct.GetMemory()).Get(0));

				UTEST_TRUE(TEXT("TestContainerObject1 B Value"), 22 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, TestContainerObject1->InstancedStructContainer[0].GetMemory()).Get(0));
				UTEST_TRUE(TEXT("TestContainerObject2 B Value"), 22 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, TestContainerObject2->Container.InstancedStructContainer[0].GetMemory()).Get(0));

				UTEST_TRUE(TEXT("TestInstancedObject1 C Value"), 3 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, CName, TestInstancedObject1->InstancedStruct.GetMemory()).Get(0));
				UTEST_TRUE(TEXT("TestInstancedObject1 C Value"), !TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, CName, TestInstancedObject1->Container.InstancedStruct.GetMemory()).IsSet());
				UTEST_TRUE(TEXT("TestInstancedObject2 C Value"), !TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, CName, TestInstancedObject2->InstancedStruct.GetMemory()).IsSet());
				UTEST_TRUE(TEXT("TestInstancedObject2 C Value"), 3 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, CName, TestInstancedObject2->Container.InstancedStruct.GetMemory()).Get(0));

				UTEST_TRUE(TEXT("TestContainerObject1 C Value"), 3 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, CName, TestContainerObject1->InstancedStructContainer[0].GetMemory()).Get(0));
				UTEST_TRUE(TEXT("TestContainerObject2 C Value"), 3 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, CName, TestContainerObject2->Container.InstancedStructContainer[0].GetMemory()).Get(0));

				TValueOrError<FStructView, EPropertyBagResult> BagResult1 = TestBagObject1->InstancedPropertyBag.GetValueStruct("MyStruct1");
				UTEST_TRUE(TEXT("Bag1Result1"), BagResult1.HasValue());
				UTEST_TRUE(TEXT("BagResult1 struct"), BagResult1.GetValue().GetScriptStruct() == UserDefinedStruct);
				UTEST_TRUE(TEXT("Bag1Result1 B Value"), 22 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, BagResult1.GetValue().GetMemory()).Get(0));
				UTEST_TRUE(TEXT("Bag1Result1 C Value"), 3 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, CName, BagResult1.GetValue().GetMemory()).Get(0));
				TValueOrError<FStructView, EPropertyBagResult> BagResult2 = TestBagObject2->Container.InstancedPropertyBag.GetValueStruct("MyStruct2");
				UTEST_TRUE(TEXT("Bag1Result2"), BagResult2.HasValue());
				UTEST_TRUE(TEXT("BagResult2 struct"), BagResult2.GetValue().GetScriptStruct() == UserDefinedStruct);
				UTEST_TRUE(TEXT("Bag1Result2 B Value"), 22 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, BName, BagResult2.GetValue().GetMemory()).Get(0));
				UTEST_TRUE(TEXT("Bag1Result2 C Value"), 3 == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, CName, BagResult2.GetValue().GetMemory()).Get(0));
			}
		}

		return true;
	}

	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_UserDefinedStructReinstancerDefaultValues
		, "System.StructUtils.Reinstancer.DefaultValues"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_UserDefinedStructReinstancerDefaultValues::RunTest(const FString& Parameters)
	{
		static const FName AName("A");
		static const FName BName("B");
		static const FName CName("C");

		TStrongObjectPtr<UTestObjectThatContainsStruct> UDSContainer(NewObject<UTestObjectThatContainsStruct>());

		// Create UDS
		FInstancedPropertyBag BagA;
		BagA.AddProperties({
			{ AName, EPropertyBagPropertyType::Int32 },
			{ BName, EPropertyBagPropertyType::Int32 },
			{ CName, EPropertyBagPropertyType::Int32 },
			});
		BagA.SetValueInt32(AName, 1);
		BagA.SetValueInt32(BName, 2);
		BagA.SetValueInt32(CName, 3);

		UUserDefinedStruct* UserDefinedStruct = nullptr;
		{
			UE::StructUtils::FCreateUserDefinedStructArgs Args;
			Args.bAddPropertyIDToPropertyName = true;
			Args.bIsBlueprintType = false;
			TValueOrError<UUserDefinedStruct*, void> Result = UE::StructUtils::CreateUserDefinedStructFromDescs(UDSContainer.Get(), BagA, "UDS", Args);

			UTEST_TRUE("UUserDefinedStruct created", Result.HasValue());
			UserDefinedStruct = Result.GetValue();
			UTEST_TRUE("UUserDefinedStruct valid", UserDefinedStruct != nullptr);
		}

		// Assign to test object
		TStrongObjectPtr<UTestObjectThatContainsStruct> TestInstancedObject(NewObject<UTestObjectThatContainsStruct>());
		TestInstancedObject->InstancedStruct = UserDefinedStruct;
		TestInstancedObject->Container.InstancedStruct = UserDefinedStruct;
		TestInstancedObject->InstancedStructContainer.InsertAt(0, TArrayView<FInstancedStruct>(&TestInstancedObject->InstancedStruct, 1));
		TestInstancedObject->Container.InstancedStructContainer.InsertAt(0, TArrayView<FInstancedStruct>(&TestInstancedObject->InstancedStruct, 1));
		TestInstancedObject->InstancedPropertyBag.AddProperties({{"MyStruct1", EPropertyBagPropertyType::Struct, UserDefinedStruct}});
		TestInstancedObject->Container.InstancedPropertyBag.AddProperties({ {"MyStruct2", EPropertyBagPropertyType::Struct, UserDefinedStruct} });

		auto TestStruct = [this, TestInstancedObject, UserDefinedStruct]()
			{
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedStruct"), TestInstancedObject->InstancedStruct.GetScriptStruct() == UserDefinedStruct);
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedStruct"), TestInstancedObject->Container.InstancedStruct.GetScriptStruct() == UserDefinedStruct);
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedStructContainer"), TestInstancedObject->InstancedStructContainer[0].GetScriptStruct() == UserDefinedStruct);
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedStructContainer struct"), TestInstancedObject->Container.InstancedStructContainer[0].GetScriptStruct() == UserDefinedStruct);
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedPropertyBag"), TestInstancedObject->InstancedPropertyBag.GetValue().GetScriptStruct() != UserDefinedStruct);
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedPropertyBag"), TestInstancedObject->Container.InstancedPropertyBag.GetValue().GetScriptStruct() != UserDefinedStruct);

				TValueOrError<FStructView, EPropertyBagResult> BagResult1 = TestInstancedObject->InstancedPropertyBag.GetValueStruct("MyStruct1");
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedPropertyBag"), BagResult1.HasValue());
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedPropertyBag.GetStruct"), BagResult1.GetValue().GetScriptStruct() == UserDefinedStruct);
				TValueOrError<FStructView, EPropertyBagResult> BagResult2 = TestInstancedObject->Container.InstancedPropertyBag.GetValueStruct("MyStruct2");
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedPropertyBag"), BagResult2.HasValue());
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedPropertyBag.GetStruct"), BagResult2.GetValue().GetScriptStruct() == UserDefinedStruct);

				return true;
			};

		auto TestValue = [this, TestInstancedObject, UserDefinedStruct](FName PropertyName, int32 Value)
			{
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedStruct"), Value == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, PropertyName, TestInstancedObject->InstancedStruct.GetMemory()).Get(0));
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedStruct"), Value == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, PropertyName, TestInstancedObject->Container.InstancedStruct.GetMemory()).Get(0));
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedStructContainer"), Value == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, PropertyName, TestInstancedObject->InstancedStructContainer[0].GetMemory()).Get(0));
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedStructContainer"), Value == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, PropertyName, TestInstancedObject->Container.InstancedStructContainer[0].GetMemory()).Get(0));

				TValueOrError<FStructView, EPropertyBagResult> BagResult1 = TestInstancedObject->InstancedPropertyBag.GetValueStruct("MyStruct1");
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedPropertyBag"), Value == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, PropertyName, BagResult1.GetValue().GetMemory()).Get(0));
				TValueOrError<FStructView, EPropertyBagResult> BagResult2 = TestInstancedObject->Container.InstancedPropertyBag.GetValueStruct("MyStruct2");
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedPropertyBag"), Value == TryGetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, PropertyName, BagResult2.GetValue().GetMemory()).Get(0));

				return true;
			};

		auto SetValue = [this, TestInstancedObject, UserDefinedStruct](FName PropertyName, int32 Value)
			{
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedStruct"), TrySetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, PropertyName, TestInstancedObject->InstancedStruct.GetMutableMemory(), Value));
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedStruct"), TrySetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, PropertyName, TestInstancedObject->Container.InstancedStruct.GetMutableMemory(), Value));
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedStructContainer"), TrySetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, PropertyName, TestInstancedObject->InstancedStructContainer[0].GetMemory(), Value));
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedStructContainer struct"), TrySetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, PropertyName, TestInstancedObject->Container.InstancedStructContainer[0].GetMemory(), Value));

				TValueOrError<FStructView, EPropertyBagResult> BagResult1 = TestInstancedObject->InstancedPropertyBag.GetValueStruct("MyStruct1");
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedPropertyBag"), TrySetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, PropertyName, BagResult1.GetValue().GetMemory(), Value));
				TValueOrError<FStructView, EPropertyBagResult> BagResult2 = TestInstancedObject->Container.InstancedPropertyBag.GetValueStruct("MyStruct2");
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedPropertyBag"), TrySetUDSPropertyValue_InContainer<int32>(UserDefinedStruct, PropertyName, BagResult2.GetValue().GetMemory(), Value));

				return true;
			};

		UTEST_TRUE(TEXT("Initial Test Struct"), TestStruct());
		UTEST_TRUE(TEXT("Initial Test Values"), TestValue(AName, 1));
		UTEST_TRUE(TEXT("Initial Test Values"), TestValue(BName, 2));
		UTEST_TRUE(TEXT("Initial Test Values"), TestValue(CName, 3));

		{
			BagA.SetValueInt32(BName, -2);
			const TValueOrError<bool, void> ModifyResult = UE::StructUtils::UpdateUserDefinedStructFromDescs(UserDefinedStruct, BagA);
			UTEST_TRUE("UUserDefinedStruct modified", ModifyResult.HasValue() && ModifyResult.GetValue());
		}

		UTEST_TRUE(TEXT("Initial Test Struct"), TestStruct());
		UTEST_TRUE(TEXT("Initial Test Values"), TestValue(AName, 1));
		UTEST_TRUE(TEXT("Initial Test Values"), TestValue(BName, -2));
		UTEST_TRUE(TEXT("Initial Test Values"), TestValue(CName, 3));

		// Set the value of C
		{
			UTEST_TRUE(TEXT("Initial Test Values"), SetValue(CName, 33));
			UTEST_TRUE(TEXT("Initial Test Values"), TestValue(CName, 33));
		}

		// Set the default value of C. Should stay has it is. The child already modified it.
		{
			BagA.SetValueInt32(CName, -3);
			const TValueOrError<bool, void> ModifyResult = UE::StructUtils::UpdateUserDefinedStructFromDescs(UserDefinedStruct, BagA);
			UTEST_TRUE("UUserDefinedStruct modified", ModifyResult.HasValue() && ModifyResult.GetValue());
		}

		UTEST_TRUE(TEXT("Initial Test Struct"), TestStruct());
		UTEST_TRUE(TEXT("Initial Test Values"), TestValue(AName, 1));
		UTEST_TRUE(TEXT("Initial Test Values"), TestValue(BName, -2));
		UTEST_TRUE(TEXT("Initial Test Values"), TestValue(CName, 33));

		return true;
	}

	// A FInstancedStruct is an instance of a UPropertyBag.
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTest_PropertyBagReinstancer
		, "System.StructUtils.Reinstancer.PropertyBag"
		, EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
	bool FTest_PropertyBagReinstancer::RunTest(const FString& Parameters)
	{
		static const FName AName("A");
		static const FName BName("B");
		static const FName CName("C");
		static const FName DName("D");

		TStrongObjectPtr<UTestObjectThatContainsStruct> UDSContainer(NewObject<UTestObjectThatContainsStruct>());

		FInstancedPropertyBag BagA;
		BagA.AddProperties({ { CName, EPropertyBagPropertyType::Int32 } });
		BagA.SetValueInt32(CName, 3);

		// Create UDS
		UUserDefinedStruct* UserDefinedStruct = nullptr;
		{
			UE::StructUtils::FCreateUserDefinedStructArgs Args;
			Args.bAddPropertyIDToPropertyName = true;
			Args.bIsBlueprintType = false;
			TValueOrError<UUserDefinedStruct*, void> Result = UE::StructUtils::CreateUserDefinedStructFromDescs(UDSContainer.Get(), BagA, "UDS", Args);

			UTEST_TRUE("UUserDefinedStruct created", Result.HasValue());
			UserDefinedStruct = Result.GetValue();
			UTEST_TRUE("UUserDefinedStruct valid", UserDefinedStruct != nullptr);
		}

		// Create a property bag to keep it's reference
		FInstancedPropertyBag BagB;
		BagB.AddProperties({
			{ AName, EPropertyBagPropertyType::Int32 },
			{ BName, EPropertyBagPropertyType::Int32 },
			{"MyStruct", EPropertyBagPropertyType::Struct, UserDefinedStruct}
			});
		BagB.SetValueInt32(AName, 1);
		BagB.SetValueInt32(BName, 2);

		// Assign the property bag generated struct to a test object
		TStrongObjectPtr<UTestObjectThatContainsStruct> TestInstancedObject(NewObject<UTestObjectThatContainsStruct>());
		TestInstancedObject->InstancedStruct = BagB.GetValue();
		FConstStructView BagBValue = BagB.GetValue();
		TestInstancedObject->InstancedStructContainer.InsertAt(0, TArrayView<FConstStructView>(&BagBValue, 1));

		auto TestStruct = [this, TestInstancedObject](TNotNull<const UScriptStruct*> ScriptStruct, TNotNull<const UScriptStruct*> MyStruct)
			{
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedStruct"), TestInstancedObject->InstancedStruct.GetScriptStruct() == ScriptStruct);
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedStructContainer"), TestInstancedObject->InstancedStructContainer[0].GetScriptStruct() == ScriptStruct);
				const FProperty* Property = ScriptStruct->FindPropertyByName("MyStruct");
				UTEST_TRUE(TEXT("Property"), Property != nullptr);
				const FStructProperty* StructProperty = CastField<const FStructProperty>(Property);
				UTEST_TRUE(TEXT("StructProperty"), StructProperty != nullptr);
				if (StructProperty == nullptr) // C6011
				{
					return false;
				}
				UTEST_TRUE(TEXT("StructProperty == MyStruct"), MyStruct == StructProperty->Struct);
				return true;
			};

		auto TestValue = [this, TestInstancedObject](FName PropertyName, int32 Value)
			{
				const UScriptStruct* ScriptStruct = TestInstancedObject->InstancedStruct.GetScriptStruct();
				UTEST_TRUE(TEXT("TestInstancedObject.InstancedStruct"), Value == TryGetPropertyValue_InContainer<int32>(ScriptStruct, PropertyName, TestInstancedObject->InstancedStruct.GetMemory()).Get(0));
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedStruct"), Value == TryGetPropertyValue_InContainer<int32>(ScriptStruct, PropertyName, TestInstancedObject->InstancedStructContainer[0].GetMemory()).Get(0));
				return true;
			};

		auto TestPropertyCDValue = [this, TestInstancedObject](int32 Value, int32 Index)
			{
				const UScriptStruct* ScriptStruct = TestInstancedObject->InstancedStruct.GetScriptStruct();
				const FStructProperty* StructProperty = CastField<const FStructProperty>(ScriptStruct->FindPropertyByName("MyStruct"));
				if (StructProperty == nullptr) // C6011
				{
					return false;
				}

				// The struct is 2 integers (C and D).
				void* StructData = FMemory::Malloc(StructProperty->Struct->GetStructureSize());
				StructProperty->Struct->InitializeStruct(StructData);

				StructProperty->GetValue_InContainer(TestInstancedObject->InstancedStruct.GetMemory(), StructData);
				int32 CStructValue1 = reinterpret_cast<int32*>(StructData)[Index];
				StructProperty->GetValue_InContainer(TestInstancedObject->InstancedStructContainer[0].GetMemory(), StructData);
				int32 CStructValue2 = reinterpret_cast<int32*>(StructData)[Index];

				StructProperty->Struct->DestroyStruct(StructData);
				FMemory::Free(StructData);

				UTEST_TRUE(TEXT("TestInstancedObject.InstancedStruct"), Value == CStructValue1);
				UTEST_TRUE(TEXT("TestInstancedObject.Container.InstancedStruct"), Value == CStructValue2);

				return true;
			};

		UTEST_TRUE(TEXT("Initial Test Struct"), TestStruct(BagB.GetValue().GetScriptStruct(), UserDefinedStruct));
		UTEST_TRUE(TEXT("Initial Test Values"), TestValue(AName, 1));
		UTEST_TRUE(TEXT("Initial Test Values"), TestValue(BName, 2));
		UTEST_TRUE(TEXT("Initial Test Values"), TestPropertyCDValue(3, 0));

		// Set the value of C.
		// TestInstancedObject->InstancedStruct is not a FInstancedPropertyBag. It wrongly point to the temporary UPropertyBag struct.
		//This is not supported but we do not want to crash on dangling memory.
		{
			UTEST_TRUE(TEXT("Assign"), EPropertyBagResult::Success == BagA.SetValueInt32(CName, -3));
			BagA.AddProperties({ { DName, EPropertyBagPropertyType::Int32 } });
			UTEST_TRUE(TEXT("Assign"), EPropertyBagResult::Success == BagA.SetValueInt32(DName, 4));
			const TValueOrError<bool, void> ModifyResult = UE::StructUtils::UpdateUserDefinedStructFromDescs(UserDefinedStruct, BagA);
			UTEST_TRUE(TEXT("UUserDefinedStruct modified"), ModifyResult.HasValue() && ModifyResult.GetValue());
		}
		{
			const UScriptStruct* OldBagBStruct = BagB.GetValue().GetScriptStruct();
			UTEST_TRUE(TEXT("Modified Test Struct A"), OldBagBStruct->FindPropertyByName(AName) != nullptr);
			UTEST_TRUE(TEXT("Modified Test Struct B"), OldBagBStruct->FindPropertyByName(BName) != nullptr);
			const FProperty* MyStructProperty = OldBagBStruct->FindPropertyByName("MyStruct");
			UTEST_TRUE(TEXT("Modified Test Struct Struct"), MyStructProperty != nullptr);
			const FStructProperty* MyStructStructProperty = CastField<const FStructProperty>(MyStructProperty);
			UTEST_TRUE(TEXT("Modified Test Struct Struct"), MyStructStructProperty != nullptr);
			if (MyStructStructProperty == nullptr) // C6011
			{
				return false;
			}
			UTEST_TRUE(TEXT("Modified Test Struct Struct"), MyStructStructProperty->Struct != nullptr);
			UTEST_TRUE(TEXT("Modified Test Struct Newer"), (MyStructStructProperty->Struct->StructFlags & STRUCT_NewerVersionExists) != 0);
		}

		{
			const UScriptStruct* NewBagBStruct = TestInstancedObject->InstancedStruct.GetScriptStruct();
			UTEST_TRUE(TEXT("Modified Test Struct A"), NewBagBStruct->FindPropertyByName(AName) != nullptr);
			UTEST_TRUE(TEXT("Modified Test Struct B"), NewBagBStruct->FindPropertyByName(BName) != nullptr);
			const FProperty* MyStructProperty = NewBagBStruct->FindPropertyByName("MyStruct");
			UTEST_TRUE(TEXT("Modified Test Struct Struct"), MyStructProperty != nullptr);
			const FStructProperty* MyStructStructProperty = CastField<const FStructProperty>(MyStructProperty);
			UTEST_TRUE(TEXT("Modified Test Struct Struct"), MyStructStructProperty != nullptr);
			if (MyStructStructProperty == nullptr) // C6011
			{
				return false;
			}
			UTEST_TRUE(TEXT("Modified Test Struct Struct"), MyStructStructProperty->Struct != nullptr);
			UTEST_TRUE(TEXT("Modified Test Struct Newer"), (MyStructStructProperty->Struct->StructFlags & STRUCT_NewerVersionExists) == 0);

			const UUserDefinedStruct* NewUDS = Cast<UUserDefinedStruct>(MyStructStructProperty->Struct);
			UTEST_TRUE(TEXT("Modified is UDS"), NewUDS != nullptr);
			if (NewUDS == nullptr) // C6011
			{
				return false;
			}
			const UUserDefinedStructEditorData* EditorData = Cast<UUserDefinedStructEditorData>(NewUDS->EditorData);
			UTEST_TRUE(TEXT("Modified is UDS editor data"), EditorData != nullptr);
			if (EditorData == nullptr) // C6011
			{
				return false;
			}
			UTEST_TRUE(TEXT("Modified Find Struct C"), EditorData->FindProperty(NewUDS, CName) != nullptr);
			UTEST_TRUE(TEXT("Modified Find Struct D"), EditorData->FindProperty(NewUDS, DName) != nullptr);
			UTEST_TRUE(TEXT("Modified Test Values"), TestValue(AName, 1));
			UTEST_TRUE(TEXT("Modified Test Values"), TestValue(BName, 2));
			UTEST_TRUE(TEXT("Modified Test Values"), TestPropertyCDValue(-3, 0));
			UTEST_TRUE(TEXT("Modified Test Values"), TestPropertyCDValue(4, 1));
		}

		return true;
	}

} // UE::StructUtils::Private

#undef LOCTEXT_NAMESPACE
