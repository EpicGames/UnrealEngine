// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/InstancedStructContainer.h"
#include "StructUtils/PropertyBag.h"
#include "StructUtils/UserDefinedStruct.h"
#include "StructReinstancerTests.generated.h"

namespace UE::StructUtils::Private
{
USTRUCT()
struct FTestStructThatContainsStruct
{
	GENERATED_BODY()

	UPROPERTY()
	FVector Vect = FVector::ZeroVector;

	UPROPERTY()
	FInstancedStruct InstancedStruct;

	UPROPERTY()
	FInstancedStructContainer InstancedStructContainer;

	UPROPERTY()
	FInstancedPropertyBag InstancedPropertyBag;
};

UCLASS()
class UTestObjectThatContainsStructBase : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FVector Vect = FVector::ZeroVector;

	UPROPERTY()
	FTestStructThatContainsStruct Container;

	UPROPERTY()
	FInstancedStruct InstancedStruct;

	UPROPERTY()
	FInstancedStructContainer InstancedStructContainer;

	UPROPERTY()
	FInstancedPropertyBag InstancedPropertyBag;
};

UCLASS()
class UTestObjectThatContainsStruct : public UTestObjectThatContainsStructBase
{
	GENERATED_BODY()
};
}