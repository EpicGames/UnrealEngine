// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataHierarchyViewModelBase.h"
#include "PropertyEditorArchetypePolicy.h"
#include "Math/UnitConversion.h"
#include "StructUtils/InstancedStructContainer.h"
#include "StructUtils/PropertyBag.h"
#include "PropertyBagHierarchyViewModel.generated.h"

struct FPropertyBagHierarchyPropertyViewModel;
class UPropertyBagSchema;

/** This file defines hierarchy elements and the view model that controls the hierarchy.
 *  If you are looking to make use of them, simply subclass UPropertyBagSchema and override relevant functions.
 *  If you wish to add properties to your usecase for editor purposes, consider subclassing either property, section or category.
 *  If your property is generic and should be functioning for every property bag, consider adding them in the base classes here.
 */
DECLARE_MULTICAST_DELEGATE(FOnPropertyBagHierarchyModified);

UCLASS(MinimalAPI)
class UPropertyBagHierarchyRoot : public UHierarchyRoot
{
	GENERATED_BODY()
	
public:
	FOnPropertyBagHierarchyModified OnHierarchyModified;
};

UCLASS(MinimalAPI)
class UPropertyBagHierarchyCategory : public UHierarchyCategory
{
	GENERATED_BODY()
	
public:
	// Add generic future properties for the category here
};

UCLASS(MinimalAPI)
class UPropertyBagHierarchySection : public UHierarchySection
{
	GENERATED_BODY()
	
public:
	// Add generic future properties for the section here
};

USTRUCT()
struct FPropertyBagPropertyMetadata_Base
{
	GENERATED_BODY()
	
	virtual ~FPropertyBagPropertyMetadata_Base() = default;
	
	virtual void PostEditChange(const FPropertyChangedEvent& PropertyChangedEvent, TSharedPtr<FPropertyBagHierarchyPropertyViewModel> PropertyViewModel) {}
};

USTRUCT(DisplayName="Common")
struct FPropertyBagPropertyMetadata_Common : public FPropertyBagPropertyMetadata_Base
{
	GENERATED_BODY()
	
	/** The tooltip for this property. */
	UPROPERTY(EditAnywhere, Category="Generic", meta=(MultiLine))
	FText Tooltip;
	
	/** Whether this property is advanced or not. If it's advanced, it will only display if the "Show Advanced" button is clicked. */
	UPROPERTY(EditAnywhere, Category="Generic")
	bool bAdvanced = false;
	
	virtual void PostEditChange(const FPropertyChangedEvent& PropertyChangedEvent, TSharedPtr<FPropertyBagHierarchyPropertyViewModel> PropertyViewModel) override;
};

USTRUCT(DisplayName="Numeric")
struct FPropertyBagPropertyMetadata_Numeric : public FPropertyBagPropertyMetadata_Base
{
	GENERATED_BODY()
	
	static constexpr float DefaultClampMin = 0.f;
	static constexpr float DefaultClampMax = 10.f;
	static constexpr float DefaultUIMin = 0.f;
	static constexpr float DefaultUIMax = 10.f;
	static constexpr EUnit DefaultUnit = EUnit::Unspecified;
	
	UPROPERTY()
	bool bUseClampMin = false;
	
	UPROPERTY(EditAnywhere, Category="Numeric", meta=(EditCondition="bUseClampMin"))
	float ClampMin = DefaultClampMin;

	UPROPERTY()
	bool bUseClampMax = false;
	
	UPROPERTY(EditAnywhere, Category="Numeric", meta=(EditCondition="bUseClampMax"))
	float ClampMax = DefaultClampMax;
	
	UPROPERTY()
	bool bUseUIMin = false;
	
	UPROPERTY(EditAnywhere, Category="Numeric", meta=(EditCondition="bUseUIMin"))
	float UIMin = DefaultUIMin;
	
	UPROPERTY()
	bool bUseUIMax = false;
	
	UPROPERTY(EditAnywhere, Category="Numeric", meta=(EditCondition="bUseUIMax"))
	float UIMax = DefaultUIMax;
	
	UPROPERTY()
	bool bUseUnit = false;
	
	UPROPERTY(EditAnywhere, Category="Numeric", meta=(EditCondition="bUseUnit"))
	EUnit Unit = DefaultUnit;
	
	virtual void PostEditChange(const FPropertyChangedEvent& PropertyChangedEvent, TSharedPtr<FPropertyBagHierarchyPropertyViewModel> PropertyViewModel) override;
};

UCLASS(MinimalAPI)
class UPropertyBagHierarchyProperty : public UHierarchyItem
{
	GENERATED_BODY()
	
public:
	void Initialize(const FPropertyBagPropertyDesc& PropertyDesc);
	
	UPropertyBagHierarchyProperty* GetArchetype() const;
	
	static FHierarchyElementIdentity ConstructIdentity(const FPropertyBagPropertyDesc& PropertyDesc);
	
	FGuid GetPropertyId() const;

	FInstancedStruct* FindOrAddPropertyMetaData(const UScriptStruct* InScriptStruct);
	
	bool ContainsPropertyMetaDataOfType(const UScriptStruct* InScriptStruct);
	
	template<typename T>
	bool ContainsPropertyMetaDataOfType()
	{
		static_assert(TIsDerivedFrom<T, FPropertyBagPropertyMetadata_Base>::IsDerived, "MetaData must be of type FPropertyBagPropertyMetadata_Base!");
		return ContainsPropertyMetaDataOfType(T::StaticStruct);
	}
	
	const FInstancedStruct* FindPropertyMetaDataOfType(const UScriptStruct* InScriptStruct) const;
	
	FInstancedStruct* FindPropertyMetaDataOfTypeMutable(const UScriptStruct* InScriptStruct);
	
	template<typename T>
	const T* FindPropertyMetaDataOfType() const
	{
		static_assert(TIsDerivedFrom<T, FPropertyBagPropertyMetadata_Base>::IsDerived, "MetaData must be of type FPropertyBagPropertyMetadata_Base!");
		
		if (const FInstancedStruct* Result = FindPropertyMetaDataOfType(T::StaticStruct()))
		{
			return Result->GetPtr<T>();
		}
		
		return nullptr;
	}
	
	template<typename T>
	T* FindPropertyMetaDataOfTypeMutable()
	{
		static_assert(TIsDerivedFrom<T, FPropertyBagPropertyMetadata_Base>::IsDerived, "MetaData must be of type FPropertyBagPropertyMetadata_Base!");
		
		if (FInstancedStruct* Result = FindPropertyMetaDataOfTypeMutable(T::StaticStruct()))
		{
			return Result->GetMutablePtr<T>();
		}
		
		return nullptr;
	}
	
	/** All metadata goes in here in form of FInstancedStruct. This is maintained by the API. */
	UPROPERTY(EditAnywhere, Category="NoCategory")
	FInstancedStructArray PropertyMetadata;
};

struct FPropertyBagHierarchyPropertyViewModel : public FHierarchyItemViewModel
{
	FPropertyBagHierarchyPropertyViewModel(UPropertyBagHierarchyProperty* InProperty, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel);
	
	void UpdateMetaData();
	
	const FPropertyBagPropertyDesc* GetPropertyDesc() const;
	
	virtual void PostEditChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	virtual bool RepresentsExternalData() const override { return true; }
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const override;
	virtual void SyncViewModelsToDataInternal() override;
	
	/** Users can't manually delete the properties. They can still be deleted if their parent gets deleted however.
	 *  We re-add the properties to the root in that case. */
	virtual bool CanDeleteInternal() override { return false; }
	
protected:
	virtual FString ToString() const override;
};

/** The hierarchy view model for property bags. Defines core rules for how the hierarchy behaves. */
UCLASS(MinimalAPI)
class UPropertyBagHierarchyViewModel : public UDataHierarchyViewModelBase, public FTickableEditorObject
{
	GENERATED_BODY()
	
public:
	/**
	 * Initializes the hierarchy view model. Internally creates a PropertyRowGenerator to generate a stable property handle
	 * that survives details panel refreshes. The property handle is used by ApplyChangesToSinglePropertyDesc to notify
	 * pre/post change when editing property metadata (tooltips, clamp values, etc.) via the hierarchy editor.
	 *
	 * Note: PropertyUtilities must be set separately via SetPropertyUtilities() after Initialize returns.
	 * This is because Initialize triggers a PropertyRowGenerator layout rebuild which re-enters AcquireHierarchyViewModel,
	 * and setting utilities during that reentrancy would overwrite them with the generator's internal utilities.
	 *
	 * @param InOwningObject The UObject that owns the property bag (e.g. the Blueprint CDO or actor). Used to populate the PropertyRowGenerator.
	 * @param InPropertyPath Path to the FInstancedPropertyBag property within the owning object. Used to locate the property bag handle in the generated property tree.
	 */
	void Initialize(UObject* InOwningObject, TSharedRef<FPropertyPath> InPropertyPath);
	
	virtual void BeginDestroy() override;
	
	void RegeneratePropertyBagHandle();
	
	void AddMissingPropertiesToRoot(bool bSendNotification = false);
	
	const FPropertyBagPropertyDesc* GetPropertyDescForProperty(FGuid PropertyId) const;
	TConstArrayView<FPropertyBagPropertyDesc> GetPropertyDescs() const;
	
	TSharedPtr<IPropertyHandle> GetPropertyBagHandle() const { return GeneratedPropertyBagHandle; }
	TSharedPtr<IPropertyUtilities> GetPropertyUtilities() const { return PropertyUtilities.IsValid() ? PropertyUtilities.Pin() : nullptr; }
	TSharedRef<FPropertyPath> GetCurrentPropertyPath() const { return PropertyPathToPropertyBag.ToSharedRef(); }

	/** Updates the property utilities reference. Call when a new details panel instance provides fresh utilities. */
	void SetPropertyUtilities(TSharedRef<IPropertyUtilities> InPropertyUtilities);

	void RebuildDetails() const;

	virtual UHierarchyRoot* GetHierarchyRoot() const override;

protected:
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override;
	virtual TStatId GetStatId() const override;
	
	virtual void OnHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload) override;
	virtual TSubclassOf<UHierarchyCategory> GetCategoryDataClass() const override;
	virtual TSubclassOf<UHierarchySection> GetSectionDataClass() const override;
	virtual TSharedPtr<FHierarchyElementViewModel> CreateCustomViewModelForElement(UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> Parent) override;
	/** We auto-add elements to the hierarchy, so we don't need the source panel. */
	virtual bool SupportsSourcePanel() const override { return false; }
	virtual ESelectionMode::Type GetHierarchySelectionMode() const override { return ESelectionMode::Multi; }
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	
	const FInstancedPropertyBag* GetPropertyBag() const;

	static bool FindPropertyHandleRecursive(const TSharedPtr<IPropertyHandle>& PropertyHandle, TSharedRef<FPropertyPath> PropertyPath);
	static TSharedPtr<IDetailTreeNode> FindTreeNodeRecursive(const TSharedRef<IDetailTreeNode>& RootNode, TSharedRef<FPropertyPath> PropertyPath);
	static TSharedPtr<IDetailTreeNode> FindNode(const TArray<TSharedRef<IDetailTreeNode>>& RootNodes, TSharedRef<FPropertyPath> PropertyPath);
	
private:
	TWeakPtr<IPropertyUtilities> PropertyUtilities;
	TWeakObjectPtr<const UPropertyBagSchema> PropertyBagSchema;
	
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;
	TWeakObjectPtr<UHierarchyRoot> HierarchyRoot;
	TWeakObjectPtr<UObject> OwningObject;
	TSharedPtr<FPropertyPath> PropertyPathToPropertyBag;
	
	uint64 LastHash = 0;
	
	TSharedPtr<IPropertyHandle> GeneratedPropertyBagHandle;
};

struct FPropertyBagHierarchyScriptRootViewModel : public FHierarchyRootViewModel
{
	FPropertyBagHierarchyScriptRootViewModel(UHierarchyRoot* Root, TWeakObjectPtr<UPropertyBagHierarchyViewModel> ViewModel, bool bInIsForHierarchy)
		: FHierarchyRootViewModel(Root, ViewModel, bInIsForHierarchy)
	{}

	STRUCTUTILSEDITOR_API virtual void PostEditHierarchyStructure() override;
};

class FPropertyBagHierarchyViewModelPropertyEditorPolicy : public PropertyEditorPolicy::IArchetypePolicy
{
public:
	FPropertyBagHierarchyViewModelPropertyEditorPolicy();

	virtual ~FPropertyBagHierarchyViewModelPropertyEditorPolicy();

	virtual UObject* GetArchetypeForObject(const UObject* Object) const override;
};