// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "UObject/Interface.h"

#include "IPropertyBagEditorGraph.generated.h"

/**
 * PropertyBag Editor public interface
 *
 * Details: IPropertyBagEdGraph is an interface that enables all interactivity between Property Bags and any
 * Editor Graph.
 *
 * Usage: Inherit from `public IPropertyBagEdGraph` along with UEdGraph (or subclass of) in any distinct implementation
 * of UEdGraph. See implementations below for more details.
 *
 * Current Implementations:
 *   Drag and Drop Operations from a Property Bag Details View Child Row
 */

struct FPropertyBagPropertyDesc;
class SWidget;
class UEdGraphPin;
class UEdGraphNode;
class UEdGraph;

UINTERFACE(MinimalAPI)
class UPropertyBagEdGraphDragAndDrop : public UInterface
{
	GENERATED_BODY()
};

/**
 * PropertyBag Editor Drag and Drop public interface.
 *
 * Usage: This interface can be used independently or as a part of the IPropertyBagEdGraph interface (below), to be
 * inherited by custom subclasses of UEdGraph. Once inherited, implement the following member overrides to validate and
 * receive Drag and Drop operations from a PropertyBag details child row. The PropertyBagDragDropHandler
 * (see PropertyBagDragDropHandler.h/.cpp) inherits from FGraphEditorDragDropAction, mimicking a graph schema action
 * being dropped into the graph, or onto a pin or node.
 *
 * Note: The base handler class currently climbs down ownership from Graph->Pin, so if only the graph is valid, it will
 * attempt the graph panel drop through the pins' or nodes' parent graph.
 */
class IPropertyBagEdGraphDragAndDrop
{
	GENERATED_BODY()

public:
	virtual bool CanReceivePropertyBagDetailsDropOnGraphPin(const UEdGraphPin* Pin) const;
	virtual bool CanReceivePropertyBagDetailsDropOnGraphNode(const UEdGraphNode* Node) const;
	virtual bool CanReceivePropertyBagDetailsDropOnGraph(const UEdGraph* Graph) const;
	virtual FReply OnPropertyBagDetailsDropOnGraphPin(const FPropertyBagPropertyDesc& PropertyDesc, UEdGraphPin* Pin, const FVector2f& GraphPosition) const;
	virtual FReply OnPropertyBagDetailsDropOnGraphNode(const FPropertyBagPropertyDesc& PropertyDesc, UEdGraphNode* Node, const FVector2f& GraphPosition) const;
	virtual FReply OnPropertyBagDetailsDropOnGraph(const FPropertyBagPropertyDesc& PropertyDesc, UEdGraph* Graph, const FVector2f& GraphPosition) const;
};

UINTERFACE(MinimalAPI)
class UPropertyBagEdGraph : public UPropertyBagEdGraphDragAndDrop
{
	GENERATED_BODY()
};

/**
 * PropertyBag Editor public interface
 *
 * Usage: This interface to be inherited by custom subclasses of UEdGraph for full implementation of all StructUtils
 * editor graph integration features.
 */
class IPropertyBagEdGraph : public IPropertyBagEdGraphDragAndDrop
{
	GENERATED_BODY()
};

inline bool IPropertyBagEdGraphDragAndDrop::CanReceivePropertyBagDetailsDropOnGraphPin(const UEdGraphPin* Pin) const
{
	return false;
}

inline bool IPropertyBagEdGraphDragAndDrop::CanReceivePropertyBagDetailsDropOnGraphNode(const UEdGraphNode* Node) const
{
	return false;
}

inline bool IPropertyBagEdGraphDragAndDrop::CanReceivePropertyBagDetailsDropOnGraph(const UEdGraph* Graph) const
{
	return false;
}

inline FReply IPropertyBagEdGraphDragAndDrop::OnPropertyBagDetailsDropOnGraphPin(const FPropertyBagPropertyDesc& PropertyDesc, UEdGraphPin* Pin, const FVector2f& GraphPosition) const
{
	return FReply::Handled();
}

inline FReply IPropertyBagEdGraphDragAndDrop::OnPropertyBagDetailsDropOnGraphNode(const FPropertyBagPropertyDesc& PropertyDesc, UEdGraphNode* Node, const FVector2f& GraphPosition) const
{
	return FReply::Handled();
}

inline FReply IPropertyBagEdGraphDragAndDrop::OnPropertyBagDetailsDropOnGraph(const FPropertyBagPropertyDesc& PropertyDesc, UEdGraph* Graph, const FVector2f& GraphPosition) const
{
	return FReply::Handled();
}
