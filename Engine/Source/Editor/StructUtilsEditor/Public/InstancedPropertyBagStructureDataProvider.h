// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStructureDataProvider.h"
#include "StructUtils/PropertyBag.h"

//	FInstancePropertyBagStructureDataProvider
//	Allows structure views to use FInstancedPropertyBag even if the bag layout changes.
//	The caller needs to make sure that property bag outlives the property view widget.
class FInstancePropertyBagStructureDataProvider : public IStructureDataProvider
{
public:
	FInstancePropertyBagStructureDataProvider(FInstancedPropertyBag& InPropertyBag)
		: PropertyBag(InPropertyBag)
	{
	}

	virtual bool IsValid() const override
	{
		return PropertyBag.IsValid();
	};
	
	virtual const UStruct* GetBaseStructure() const override
	{
		return PropertyBag.IsValid() ? PropertyBag.GetPropertyBagStruct() : nullptr;
	}
	
	virtual void GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* ExpectedBaseStructure) const override
	{
		if (PropertyBag.IsValid())
		{
			const UStruct* Struct = PropertyBag.GetPropertyBagStruct();
			if (ExpectedBaseStructure && Struct && Struct == ExpectedBaseStructure)
			{
				OutInstances.Add(MakeShared<FStructOnScope>(Struct, PropertyBag.GetMutableValue().GetMemory()));
			}
		}
	}

protected:
	FInstancedPropertyBag& PropertyBag;
};

//	TInstancePropertyBagStructureDataProvider
//	Allows structure views to use FInstancedPropertyBag even if the bag layout changes.
//	The caller needs to make sure that property bag outlives the property view widget.
//	This version enables a single structure with multiple instances, and the use of
//  FInstancedPropertyBag derived types

template<typename BagInstanceType>
class TInstancedPropertyBagStructureDataProvider : public IStructureDataProvider
{
public:
	explicit TInstancedPropertyBagStructureDataProvider(const TSharedPtr<BagInstanceType>& InPropertyBag)
		: PropertyBagInstances( {InPropertyBag} )
	{
	}

	explicit TInstancedPropertyBagStructureDataProvider(const TArray<TSharedPtr<BagInstanceType>>& InPropertyBagInstances)
		: PropertyBagInstances(InPropertyBagInstances)
	{
	}

	virtual bool IsValid() const override
	{
		const UStruct* BaseStructure = GetBaseStructure();
		if (BaseStructure != nullptr)
		{
			for (const TSharedPtr<BagInstanceType>& BagInstance : PropertyBagInstances)
			{
				if (BagInstance.IsValid() && BagInstance->IsValid())
				{
					const UStruct* Struct = BagInstance->GetPropertyBagStruct();
					if (Struct->IsChildOf(BaseStructure))
					{
						return true;
					}
				}
			}
		}

		return false;
	}

	virtual const UStruct* GetBaseStructure() const override
	{
		return (PropertyBagInstances.Num() > 0) 
			? FStructOnScopeStructureDataProvider::FindBaseStructure(PropertyBagInstances)
			: nullptr;
	}

	virtual void GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* ExpectedBaseStructure) const override
	{
		if (PropertyBagInstances.Num() > 0 && ExpectedBaseStructure != nullptr)
		{
			for (auto& BagInstance : PropertyBagInstances)
			{
				if (BagInstance.IsValid() && BagInstance->IsValid())
				{
					const UStruct* Struct = BagInstance->GetPropertyBagStruct();
					if (Struct && Struct->IsChildOf(ExpectedBaseStructure))
					{
						OutInstances.Add(MakeShared<FStructOnScope>(Struct, BagInstance->GetMutableValue().GetMemory()));
					}
				}
			}
		}
	}

protected:
	TArray<TSharedPtr<BagInstanceType>> PropertyBagInstances;
};
