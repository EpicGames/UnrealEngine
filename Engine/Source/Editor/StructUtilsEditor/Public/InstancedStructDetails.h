// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "IDetailCustomNodeBuilder.h"
#include "IStructureDataProvider.h"
#include "PropertyCustomizationHelpers.h"

#define UE_API STRUCTUTILSEDITOR_API

class IAssetReferenceFilter;
class IPropertyHandle;
class IDetailGroup;
class IDetailPropertyRow;
class IPropertyHandle;
class FStructOnScope;
class SWidget;
class SInstancedStructPicker;
struct FInstancedStruct;
class FInstancedStructProvider;

/**
 * Type customization for FInstancedStruct.
 */
class FInstancedStructDetails : public IPropertyTypeCustomization
{
public:
	UE_API virtual ~FInstancedStructDetails() override;
	
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	UE_API virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	UE_API void OnCopyInstancedStruct() const;

	UE_API bool CanPasteInstancedStruct() const;
	UE_API void OnPasteInstancedStruct();

	using FReplacementObjectMap = TMap<UObject*, UObject*>;
	UE_API void OnObjectsReinstanced(const FReplacementObjectMap& ObjectMap);

	/** Handle to the struct property being edited */
	TSharedPtr<IPropertyHandle> StructProperty;

	TSharedPtr<SInstancedStructPicker> StructPicker;
	TSharedPtr<IPropertyUtilities> PropUtils;
	
	FDelegateHandle OnObjectsReinstancedHandle;
};

/** 
 * Node builder for FInstancedStruct children.
 * Expects property handle holding FInstancedStruct as input.
 * Can be used in a implementation of a IPropertyTypeCustomization CustomizeChildren() to display editable FInstancedStruct contents.
 * OnChildRowAdded() is called right after each property is added, which allows the property row to be customizable.
 * Child properties will be grouped if they 1) have "Category" metadata, and 2) have the "EnableCategories" metadata tag.
 */
class FInstancedStructDataDetails : public IDetailCustomNodeBuilder, public TSharedFromThis<FInstancedStructDataDetails>
{
public:
	UE_API FInstancedStructDataDetails(TSharedPtr<IPropertyHandle> InStructProperty);
	UE_API virtual ~FInstancedStructDataDetails() override;

	//~ Begin IDetailCustomNodeBuilder interface
	UE_API virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override;
	UE_API virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	UE_API virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	UE_API virtual void Tick(float DeltaTime) override;
	virtual bool RequiresTick() const override { return true; }
	virtual bool InitiallyCollapsed() const override { return false; }
	UE_API virtual FName GetName() const override;
	//~ End IDetailCustomNodeBuilder interface

	// Called when a group is added, override to customize a group row.
	virtual void OnGroupRowAdded(IDetailGroup& GroupRow, int32 Level, const FString& Category) const {}
	// Called when a child is added, override to customize a child row.
	virtual void OnChildRowAdded(IDetailPropertyRow& ChildRow) {}

protected:
	UE_API static TArray<FString> GetPropertyCategories(TSharedPtr<IPropertyHandle> PropertyHandle);
	/** Add all the rows for a child properties. */
	UE_API virtual void AddChildRows(IDetailChildrenBuilder& ChildBuilder, const TArray<TSharedPtr<IPropertyHandle>>& ChildProperties);
	/** Add all the rows for a child's property. */
	UE_API virtual void AddChildRow(IDetailChildrenBuilder& ChildBuilder, TSharedPtr<IPropertyHandle> ChildHandle, IDetailGroup* PropertyGroup);

private:
	UE_API void OnUserDefinedStructReinstancedHandle(const class UUserDefinedStruct& Struct);

	/** Pre/Post change notifications for struct value changes */
	UE_API void OnStructValuePreChange();
	UE_API void OnStructValuePostChange();
	UE_API void OnStructHandlePostChange();

	/** Returns type of the instanced struct for each instance/object being edited. */
	UE_API TArray<TWeakObjectPtr<const UStruct>> GetInstanceTypes() const;

	/**
	 * Adds groups for the specified properties. One group is created for each unique category (from property metadata) that the properties have.
	 * If a category is pipe-separated (eg, Foo|Bar), one group is added for "Foo" and another one for "Foo|Bar". In the returned map, the key is the
	 * property, and the value is the group. If the property doesn't have a group (category), then it will not have an entry in the map. Note that
	 * the property must opt-in to grouping by specifying the "EnableCategories" metadata tag.
	 */
	UE_API void GetPropertyGroups(const TArray<TSharedPtr<IPropertyHandle>>& InProperties, IDetailChildrenBuilder& InChildBuilder, TMap<TSharedPtr<IPropertyHandle>, IDetailGroup*>& OutPropertyToGroup) const;

	/** Cached instance types, used to invalidate the layout when types change. */
	TArray<TWeakObjectPtr<const UStruct>> CachedInstanceTypes;

	/** True if we're allowed to handle a StructValuePostChange */
	bool bCanHandleStructValuePostChange = false;
	/** Mark the struct has changed, it requires a rebuild on the next iteration. */
	bool bStructChangeDetected = false;
	
	FDelegateHandle UserDefinedStructReinstancedHandle;

protected:
	/** Handle to the struct property being edited */
	TSharedPtr<IPropertyHandle> StructProperty;
	
	/** Delegate that can be used to refresh the child rows of the current struct (eg, when changing struct type) */
	FSimpleDelegate OnRegenerateChildren;
	
	UE_API void OnStructLayoutChanges();
};

class FInstancedStructDataWithHeaderDetails : public FInstancedStructDataDetails
{
public:
	FInstancedStructDataWithHeaderDetails(TSharedPtr<IPropertyHandle> InStructProperty);
	virtual ~FInstancedStructDataWithHeaderDetails() override = default;
	
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
};

class FInstancedStructArrayBuilder : public FDetailArrayBuilder, public TSharedFromThis<FInstancedStructArrayBuilder>
{
public:
	FInstancedStructArrayBuilder(TSharedRef<IPropertyHandle> InArrayHandle);
	virtual ~FInstancedStructArrayBuilder() override = default;
	
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	
private:
	TSharedPtr<IPropertyHandle> ArrayHandle;
};

class FInstancedStructArrayDetails : public IPropertyTypeCustomization
{
public:
	FInstancedStructArrayDetails() = default;
	virtual ~FInstancedStructArrayDetails() override = default;
	
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	
private:
	void OnGenerateArrayElementWidget(TSharedRef<IPropertyHandle> PropertyHandle, int32 Index, IDetailChildrenBuilder& DetailChildrenBuilder);
};

class FInstancedStructProvider : public IStructureDataProvider
{
public:
	FInstancedStructProvider() = default;

	explicit FInstancedStructProvider(const TSharedPtr<IPropertyHandle>& InStructProperty)
		: StructProperty(InStructProperty)
	{
	}

	virtual ~FInstancedStructProvider() override = default; 

	UE_API void Reset();
	UE_API virtual bool IsValid() const override;
	UE_API virtual const UStruct* GetBaseStructure() const override;
	UE_API virtual void GetInstances(TArray<TSharedPtr<FStructOnScope>>& OutInstances, const UStruct* ExpectedBaseStructure) const override;
	UE_API virtual bool IsPropertyIndirection() const override;
	UE_API virtual uint8* GetValueBaseAddress(uint8* ParentValueAddress, const UStruct* ExpectedBaseStructure) const override;

protected:
	UE_API void EnumerateInstances(TFunctionRef<bool(const UScriptStruct* ScriptStruct, uint8* Memory, UPackage* Package)> InFunc) const;

	TSharedPtr<IPropertyHandle> StructProperty;
};

#undef UE_API
