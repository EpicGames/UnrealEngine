// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedStructDetails.h"
#include "EdGraphSchema_K2.h"
#include "StructUtils/PropertyBag.h"
#include "PropertyBagDetails.generated.h"

#define UE_API STRUCTUTILSEDITOR_API

struct FPropertyBagPropertyMetadata_Base;
class FReply;
enum class ESelectorType : uint8;
struct FEdGraphSchemaAction;
struct FMenuEntryParams;
struct EVisibility;

class IPropertyHandle;
class IPropertyUtilities;
class IDetailPropertyRow;
class SInlineEditableTextBlock;
class SWidget;

class UPropertyBagSchema;
class UHierarchySection;
class UPropertyBagHierarchyRoot;
class UPropertyBagHierarchyProperty;
class UPropertyBagHierarchySection;
class UPropertyBagHierarchyCategory;
class UPropertyBagHierarchyViewModel;
struct FPropertyBagHierarchyViewModelOwner;

/**
 * The property bag details child rows can be completely customized by assigning a combination of these values
 * to their FPropertyBag 'ChildRowFeatures' metadata via the meta specifier. TODO: It isn't currently possible
 * to specify bitwise operations in the metadata string, but this will be added shortly. More configurations can be
 * added in the meantime to satisfy specific needs.
 */
UENUM(meta = (Bitflags))
enum class EPropertyBagChildRowFeatures : uint64
{
	/** General Options */
	Fixed                 = 0,         // Fixed layout. No features enabled.
	Renaming              = 1ULL << 0, // The property name is editable from the details view.
	Deletion              = 1ULL << 1, // The property is able to be deleted.
	DragAndDrop           = 1ULL << 2, // Drag and dropping properties is enabled.
	CompactTypeSelector   = 1ULL << 3, // A compact type selector widget is enabled to the left of the name.
	AccessSpecifierButton = 1ULL << 4, // The property metadata can be set for public/private access.
	DropDownMenuButton    = 1ULL << 5, // Drop-down menu (down arrow button) is enabled.
	Categories            = 1ULL << 6, // Categories are enabled (UPROPERTY categories).
	AllGeneralOptions     = Renaming | Deletion | DragAndDrop | CompactTypeSelector | AccessSpecifierButton | DropDownMenuButton | Categories,

	// Insert new drop general features above this line.

	/** Menu Options (for drop-down menu or other future menus (right click) */
	Menu_TypeSelector       = 1ULL << 17, // Adds the type selector pill widget to the drop-down.
	Menu_Rename             = 1ULL << 18, // Renaming the property from the drop-down menu. Requires property renaming enabled.
	Menu_Delete             = 1ULL << 19, // Deleting the property from the drop-down menu.
	Menu_Categories         = 1ULL << 20, // Create new/remove from categories.
	Menu_MetadataSpecifiers = 1ULL << 21, // Common metadata specifier options.

	AllMenuOptions = Menu_TypeSelector | Menu_Rename | Menu_Delete | Menu_Categories | Menu_MetadataSpecifiers,

	// Insert new menu features above this line.
	Deprecated = 1ULL << 63, // To allow for deprecating older features.

	// Below are configurations for convenience. These can be set via the Metadata specifier on the property bag.
	ReadOnly = Fixed,
	// Renaming and deleting enabled, with type selection happening in the drop-down menu.
	Core     = Renaming | Deletion | DropDownMenuButton | Menu_TypeSelector | Menu_Rename | Menu_Delete,
	// Also enables the compact type selector icon, drag and drop, and categories support.
	Extended = Core | DragAndDrop | CompactTypeSelector | Categories | Menu_Categories | Menu_MetadataSpecifiers,
	// All options.
	All      = AllGeneralOptions | AllMenuOptions,
	// The default version includes deprecated UI features to support previous behavior.
	Default = Renaming | Deletion | Deprecated
};

ENUM_CLASS_FLAGS(EPropertyBagChildRowFeatures);

namespace UE::StructUtils
{
	STRUCTUTILSEDITOR_API void SetPropertyDescFromPin(FPropertyBagPropertyDesc& Desc, const FEdGraphPinType& PinType);
	STRUCTUTILSEDITOR_API FEdGraphPinType GetPropertyDescAsPin(const FPropertyBagPropertyDesc& Desc);
	STRUCTUTILSEDITOR_API bool CanCreateValidGraphPinTypeForPropertyDesc(const FPropertyBagPropertyDesc& Desc);
	/** Creates type selection pill widget. */
	STRUCTUTILSEDITOR_API TSharedRef<SWidget> CreateTypeSelectionWidget(TSharedPtr<IPropertyHandle> ChildPropertyHandle, const TSharedPtr<IPropertyHandle>& InBagStructProperty, const TSharedPtr<IPropertyUtilities>& InPropUtils, ESelectorType SelectorType, bool bAllowContainers = true, TSubclassOf<UEdGraphSchema> PropertyBagSchemaClass = nullptr);

	/** @return property bag struct common to all edited properties. */
	STRUCTUTILSEDITOR_API const UPropertyBag* GetCommonBagStruct(TSharedPtr<IPropertyHandle> StructProperty);
	
	STRUCTUTILSEDITOR_API TSubclassOf<UPropertyBagSchema> ExtractPropertyBagSchemaClass(TSharedRef<IPropertyHandle> InPropertyBagHandle);
	STRUCTUTILSEDITOR_API const UPropertyBagSchema* ExtractPropertyBagSchemaCDO(TSharedRef<IPropertyHandle> InPropertyBagHandle);
	STRUCTUTILSEDITOR_API UPropertyBagHierarchyRoot* ExtractHierarchyRoot(TSharedRef<IPropertyHandle> InPropertyBagHandle);
	
	/**
	 * Allows the mutation of the property descriptors on a Property Bag property via the provided callback.
	 * Handles the whole transaction internally, including building a new Property Bag Struct (if necessary).
	 *
	 * @param SessionName Name of the transaction
	 * @param StructProperty The Property Bag handle
	 * @param Function The callback that does the mutation
	 */
	STRUCTUTILSEDITOR_API void ApplyChangesToPropertyDescs(const FText& SessionName, const TSharedPtr<IPropertyHandle>& StructProperty, TFunctionRef<void(TArray<FPropertyBagPropertyDesc>&)> Function);

	UE_DEPRECATED(5.8, "Please use the version without the property utilities parameter.")
	STRUCTUTILSEDITOR_API void ApplyChangesToPropertyDescs(const FText& SessionName, const TSharedPtr<IPropertyHandle>& StructProperty, const TSharedPtr<IPropertyUtilities>& PropUtils, TFunctionRef<void(TArray<FPropertyBagPropertyDesc>&)> Function);

	/**
	 * Allows the mutation of a single property descriptor on a Property Bag property via the provided callback.
	 * Handles the whole transaction internally, including building a new Property Bag Struct (if necessary).
	 *
	 * @param SessionName Name of the transaction
	 * @param PropertyHandle A child handle of the Property Bag to mutate
	 * @param StructProperty The Property Bag handle
	 * @param Function The callback that does the mutation
	 */
	STRUCTUTILSEDITOR_API void ApplyChangesToSinglePropertyDesc(const FText& SessionName, const TSharedPtr<IPropertyHandle> PropertyHandle, const TSharedPtr<IPropertyHandle>& StructProperty, TFunctionRef<void(FPropertyBagPropertyDesc&)> Function);

	/**
	 * Allows the mutation of a single property descriptor on a Property Bag property via the provided callback.
	 * Handles the whole transaction internally, including building a new Property Bag Struct (if necessary).
	 *
	 * @param SessionName Name of the transaction
	 * @param PropertyDesc The child descriptor to edit
	 * @param StructProperty The Property Bag handle
	 * @param Function The callback that does the mutation
	 */
	STRUCTUTILSEDITOR_API void ApplyChangesToSinglePropertyDesc(const FText& SessionName, const FPropertyBagPropertyDesc& PropertyDesc, const TSharedPtr<IPropertyHandle>& StructProperty, TFunctionRef<void(FPropertyBagPropertyDesc&)> Function);
	
	UE_DEPRECATED(5.8, "Please use the version without the property utilities parameter.")
	STRUCTUTILSEDITOR_API void ApplyChangesToSinglePropertyDesc(const FText& SessionName, const FPropertyBagPropertyDesc& PropertyDesc, const TSharedPtr<IPropertyHandle>& StructProperty, const TSharedPtr<IPropertyUtilities>& PropUtils, TFunctionRef<void(FPropertyBagPropertyDesc&)> Function);
}

/**
 * Type customization for FInstancedPropertyBag.
 */
class FPropertyBagDetails : public IPropertyTypeCustomization
{
public:
	// Required for deprecated variable 'bFixedLayout'. TODO: Remove when removing 5.6 deprecations.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPropertyBagDetails() = default;
	~FPropertyBagDetails() = default;
	FPropertyBagDetails(const FPropertyBagDetails& Other) = delete;
	FPropertyBagDetails(FPropertyBagDetails&& Other) noexcept = delete;
	FPropertyBagDetails& operator=(const FPropertyBagDetails& Other) = delete;
	FPropertyBagDetails& operator=(FPropertyBagDetails&& Other) noexcept = delete;
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** Creates add property widget. */
	static UE_API TSharedPtr<SWidget> MakeAddPropertyWidget(TSharedPtr<IPropertyHandle> InStructProperty, TSharedPtr<IPropertyUtilities> InPropUtils, EPropertyBagPropertyType DefaultType = EPropertyBagPropertyType::Bool, FSlateColor IconColor = FSlateColor::UseForeground());

protected:
	/** IPropertyTypeCustomization interface */
	UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	UE_API virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	
	static FReply SummonHierarchyEditor(TSharedPtr<IPropertyHandle> InPropertyBagHandle, TSharedPtr<IPropertyUtilities> InPropertyUtilities);
	void NavigateToHierarchyProperty(const FPropertyBagPropertyDesc& Desc);
	
	static void ClosePropertyBagHierarchyEditorWindow();

private:
	/** Handle to the property bag being edited */
	TSharedPtr<IPropertyHandle> StructProperty;
	TSharedPtr<IPropertyUtilities> PropUtils;
	EPropertyBagPropertyType DefaultType = EPropertyBagPropertyType::Bool;
	EPropertyBagChildRowFeatures ChildRowFeatures = EPropertyBagChildRowFeatures::Default;

	// We keep track of existing window & hierarchy editor loosely so we can recycle them where appropriate
	// This also means we can only have 1 property bag hierarchy editor open at a time
	// This is because an additional window can't rely on the existing details panel and property handles too much
	static TWeakPtr<SWindow> PropertyBagHierarchyEditorWindow;
	static TWeakPtr<class SPropertyBagHierarchyEditor> PropertyBagHierarchyEditor;

	/** Shared ownership of the hierarchy view model for this customization instance's lifetime.
	 *  Released in the destructor; when the last consumer drops the owner, the view model is
	 *  finalized and marked as garbage. */
	TSharedPtr<FPropertyBagHierarchyViewModelOwner> HierarchyViewModelOwner;

	UE_DEPRECATED(5.6, "Use 'ChildRowFeatures' instead.")
	bool bFixedLayout = false;
	bool bAllowContainers = true;
	
	// Default to UPropertyBagSchema, can be set with metadata PropertyBagSchemaClass
	TWeakObjectPtr<const UPropertyBagSchema> PropertyBagSchemaCDO;
	TWeakObjectPtr<UPropertyBagHierarchyRoot> HierarchyRoot;
};

/** 
 * Node builder for FInstancedPropertyBag children.
 *  - ValueProperty is FInstancedStruct of the FInstancedPropertyBag
 *  - StructProperty is FInstancedPropertyBag
 * Can be used in a implementation of a IPropertyTypeCustomization CustomizeChildren() to display editable FInstancedPropertyBag contents.
 * Use FPropertyBagDetails::MakeAddPropertyWidget() to create the add property widget.
 * OnChildRowAdded() is called right after each property is added, which allows the property row to be customizable.
 */
class FPropertyBagInstanceDataDetails : public FInstancedStructDataDetails
{
public:
	/** Generic delegate to execute on a property description */
	DECLARE_DELEGATE_OneParam(FPropertyBagPropertyDelegate, const FPropertyBagPropertyDesc& PropertyDesc)
	
	struct FConstructParams
	{
		// Hierarchy Params are only meaningful if the owning property bag has a valid HierarchyRoot provided by a schema
		struct FHierarchyParams
		{
			FPropertyBagPropertyDelegate OnNavigateToHierarchyPropertyRequested;
			TWeakObjectPtr<UPropertyBagHierarchyViewModel> HierarchyViewModel;
		};
		
		TSharedPtr<IPropertyHandle> BagStructProperty = nullptr;
		TSharedPtr<IPropertyUtilities> PropUtils = nullptr;
		bool bAllowContainers = true;
		EPropertyBagChildRowFeatures ChildRowFeatures = EPropertyBagChildRowFeatures::Default;
		FHierarchyParams HierarchyParams;
	};

	UE_API explicit FPropertyBagInstanceDataDetails(const FConstructParams& ConstructParams);
	UE_API FPropertyBagInstanceDataDetails(TSharedPtr<IPropertyHandle> InStructProperty, const TSharedPtr<IPropertyUtilities>& InPropUtils, bool bInFixedLayout, bool bInAllowContainers = true);

	// Required for deprecated variable 'bFixedLayout' and 'bAllowArrays'. TODO: Remove when removing 5.6 deprecations.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	~FPropertyBagInstanceDataDetails() = default;
	FPropertyBagInstanceDataDetails(const FPropertyBagInstanceDataDetails& Other) = delete;
	FPropertyBagInstanceDataDetails(FPropertyBagInstanceDataDetails&& Other) noexcept = delete;
	FPropertyBagInstanceDataDetails& operator=(const FPropertyBagInstanceDataDetails& Other) = delete;
	FPropertyBagInstanceDataDetails& operator=(FPropertyBagInstanceDataDetails&& Other) noexcept = delete;
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void AddChildRows(IDetailChildrenBuilder& ChildBuilder, const TArray<TSharedPtr<IPropertyHandle>>& ChildProperties) override;
	UE_API virtual void OnGroupRowAdded(IDetailGroup& GroupRow, int32 Level, const FString& Category) const override;
	UE_API virtual void OnChildRowAdded(IDetailPropertyRow& ChildRow) override;
	
	/** Enum describing if a property is overridden, or undetermined (e.g. multiselection) */
	enum class EPropertyOverrideState
	{
		Yes,
		No,
		Undetermined,
	};

	/** Interface to allow to modify override status of a specific parameter. */
	struct IPropertyBagOverrideProvider
	{
		virtual ~IPropertyBagOverrideProvider() = default;

		virtual bool IsPropertyOverridden(const FGuid PropertyID) const = 0;
		virtual void SetPropertyOverride(const FGuid PropertyID, const bool bIsOverridden) const = 0;
	};

	/**
	 * Callback function for EnumeratePropertyBags.
	 * @return true to continue enumeration
	 */
	using EnumeratePropertyBagFuncRef = TFunctionRef<bool(const FInstancedPropertyBag& /*DefaultPropertyBag*/, FInstancedPropertyBag& /*PropertyBag*/, IPropertyBagOverrideProvider& /*OverrideProvider*/)>;

	/**
	 * Method that is called to determine if a derived class has property override logic implemented.
	 * If true is returned, the overridden class is expected to implement PreChangeOverrides(), PostChangeOverrides(), EnumeratePropertyBags().
	 * @return true of derived class has override logic implemented.
	 */
	virtual bool HasPropertyOverrides() const
	{
		return false;
	}

	/** Called before property override is changed. */
	virtual void PreChangeOverrides()
	{
		// ParentPropertyHandle would be the property that holds the property overrides. 
		//		ParentPropertyHandle->NotifyPreChange();
		checkf(false, TEXT("PreChangeOverrides() is expected to be implemented when HasPropertyOverrides() returns true."));
	}

	/** Called after property override is changed. */
	virtual void PostChangeOverrides()
	{
		// ParentPropertyHandle is be the property that holds the property overrides. 
		//		ParentPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
		//		ParentPropertyHandle->NotifyFinishedChangingProperties();
		checkf(false, TEXT("PostChangeOverrides() is expected to be implemented when HasPropertyOverrides() returns true."));
	}

	/**
	 * Called to enumerate each property bag on the property handle.
	 * The Func expects DefaultPropertyBag (the values that are override), and PropertyBag (the one that PropertyBagHandle points to),
	 * and instance of IPropertyBagOverrideProvider which is used to query if specific property is overridden, or to set the property override state.
	 */
	virtual void EnumeratePropertyBags(TSharedPtr<IPropertyHandle> PropertyBagHandle, const EnumeratePropertyBagFuncRef& Func) const
	{
		checkf(false, TEXT("EnumeratePropertyBags() is expected to be implemented when HasPropertyOverrides() returns true."));
	}

	/** @return true if property of specified child property is overridden. */
	UE_API virtual EPropertyOverrideState IsPropertyOverridden(TSharedPtr<IPropertyHandle> ChildPropertyHandle) const;

	/** Called to set the override state of specified child property. */
	UE_API virtual void SetPropertyOverride(TSharedPtr<IPropertyHandle> ChildPropertyHandle, const bool bIsOverridden);

	/** @return true if the child property has default value. */
	UE_API virtual bool IsDefaultValue(TSharedPtr<IPropertyHandle> ChildPropertyHandle) const;

	/** Called to reset the child property to default value. */
	UE_API virtual void ResetToDefault(TSharedPtr<IPropertyHandle> ChildPropertyHandle);
	
	/**~ Public Hierarchy Editor API */	
	UE_API EVisibility ShouldShowHierarchyProperty(TSharedPtr<IPropertyHandle> ChildPropertyHandle) const;
	
	UE_API TWeakObjectPtr<const UHierarchySection> GetActiveHierarchySection() const;

	/** Retrieves the hierarchy property handle map. Only meaningful if used in conjunction with a hierarchy root. */
	UE_API const TMap<TObjectKey<const UPropertyBagHierarchyProperty>, TSharedPtr<IPropertyHandle>>& GetHierarchyPropertyHandleMap() const;

	/** Retrieves the hierarchy view model, if available. */
	UE_API UPropertyBagHierarchyViewModel* GetHierarchyViewModel() const;
	/**~ End of Hierarchy Editor API */
	
protected:
	UE_DEPRECATED(5.6, "Replaced by 'CreatePropertyDetailsWidget' to allow for more customizable drop-down menu location and content.")
	UE_API TSharedRef<SWidget> OnPropertyNameContent(TSharedPtr<IPropertyHandle> ChildPropertyHandle, TSharedPtr<SInlineEditableTextBlock> InlineWidget) const;

	TSharedPtr<IPropertyHandle> BagStructProperty;
	TSharedPtr<IPropertyUtilities> PropUtils;
	
	bool bAllowContainers = true;
	EPropertyBagChildRowFeatures ChildRowFeatures = EPropertyBagChildRowFeatures::Default;

	UE_DEPRECATED(5.6, "Use 'ChildRowFeatures' instead.")
	bool bFixedLayout = false;
	UE_DEPRECATED(5.6, "Use 'bAllowContainers' instead.")
	bool bAllowArrays = true;

	UE_DEPRECATED(5.8, "Use PropertyBagSchema instead. Will be invalid. PropertyBagSchema is set automatically via property metadata instead of passed in via construction arguments now.")
	TSubclassOf<UEdGraphSchema> PropertyBagSchemaClass;
	
	// Default to UPropertyBagSchema, can be set with metadata PropertyBagSchemaClass
	TWeakObjectPtr<const UPropertyBagSchema> PropertyBagSchema;
	
	TSubclassOf<UPropertyBagSchema> GetPropertyBagSchemaClass() const;
	
	/**~ Hierarchy Editor API */
	void GenerateHierarchySectionRow(IDetailChildrenBuilder& ChildrenBuilder);
	void GenerateHierarchyRootRows(IDetailChildrenBuilder& ChildrenBuilder);
	void GenerateHierarchyLeftoverPropertyRows(IDetailChildrenBuilder& ChildrenBuilder, const TArray<TSharedPtr<IPropertyHandle>>& AllChildPropertyHandles);
	void GenerateHierarchyShowAdvancedRow(IDetailChildrenBuilder& ChildrenBuilder);
	
	const FSlateBrush* GetAdvancedImage() const;
	EVisibility GetAdvancedButtonVisibility() const;
	
	TSharedPtr<SWidget> ShowAdvancedButton;
	bool bShouldShowAdvanced = false;
	
	struct FHierarchyData
	{
		TWeakObjectPtr<UPropertyBagHierarchyRoot> HierarchyRoot;
		TWeakObjectPtr<UPropertyBagHierarchyViewModel> HierarchyViewModel;
		TMap<TObjectKey<const UPropertyBagHierarchyProperty>, TSharedPtr<IPropertyHandle>> HierarchyPropertyHandleMap;
		TMap<FString, TWeakObjectPtr<const UPropertyBagHierarchyProperty>> PropertyPathPropertyMap;
		TWeakObjectPtr<const UHierarchySection> ActiveHierarchySection;
	
		FPropertyBagPropertyDelegate OnNavigateToHierarchyPropertyRequestedDelegate;
	};
	
	FHierarchyData HierarchyData;
	/**~ End of Hierarchy Editor API */	
};

UCLASS(MinimalAPI)
class UPropertyBagDetailsExtension : public UObject
{
	GENERATED_BODY()

public:
	virtual TSharedRef<FPropertyBagInstanceDataDetails> CreateInstanceDataDetails(const FPropertyBagInstanceDataDetails::FConstructParams& Params) const
	{
		return MakeShared<FPropertyBagInstanceDataDetails>(Params);
	}
};

/**
 * Specific property bag schema to allow customizing the requirements (e.g. supported containers).
 */
UCLASS(MinimalAPI)
class UPropertyBagSchema : public UEdGraphSchema_K2
{
	GENERATED_BODY()
public:
	UE_API virtual bool SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType, const EPinContainerType& ContainerType) const override;
	
	/** Retrieves the hierarchy root from the given objects that own the property bag. Optional. */
	UE_API virtual UPropertyBagHierarchyRoot* GetHierarchyRoot(const TArray<UObject*>& ObjectsWithProperty) const;

	/** Defines the property metadata for a given property. Useful if you want to add custom data to your properties. */
	UE_API virtual TArray<UScriptStruct*> GetHierarchyPropertyMetaDataTypes(const FPropertyBagPropertyDesc& Desc) const;
	
	/** Defines the category type within the hierarchy. Useful if you want to subclass and add your own properties. */
	UE_API virtual TSubclassOf<UPropertyBagHierarchyCategory> GetHierarchyCategoryType() const;
	
	/** Defines the section type within the hierarchy. Useful if you want to subclass and add your own properties. */
	UE_API virtual TSubclassOf<UPropertyBagHierarchySection> GetHierarchySectionType() const;
	
	/** Post Load helper function. Override if you know what you are doing. */
	UE_API virtual void TransferPropertyBagMetadataIntoHierarchy(const FInstancedPropertyBag& PropertyBag, UPropertyBagHierarchyRoot& HierarchyRoot) const; 
};

#undef UE_API
