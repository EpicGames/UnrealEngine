// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"

struct FInstancedPropertyBag;
struct FPropertyBagPropertyDesc;

namespace UE::StructUtils
{
	/** Kind of difference detected between two property bag descriptors. */
	enum class EPropertyBagDiffType : uint8
	{
		/** ID present in New only. */
		Added,
		/** ID present in Old only. */
		Removed,
		/** Same ID, different Name. */
		Renamed,
		/** Same ID, !CompatibleType (or container / key types differ). */
		TypeChanged,
		/** Same ID, compatible type, value bytes differ. */
		ValueChanged,
		/** Same ID, compatible type, the "Category" metadata key differs. */
		CategoryChanged,
		/** Same ID, compatible type, a non-Category metadata key differs. */
		MetaDataChanged,
		/** Same ID, PropertyFlags differ. */
		FlagsChanged,
		/** Same ID, position relative to other surviving IDs changed. */
		Reordered,
	};

	/** Single difference between two property bag descriptors. */
	struct FPropertyBagDiffEntry
	{
		/** Stable identity of the property. Always set. */
		FGuid PropertyID;

		/** Name on the old side. NAME_None when DiffType == Added. */
		FName OldName = NAME_None;

		/** Name on the new side. NAME_None when DiffType == Removed. */
		FName NewName = NAME_None;

		/** The kind of difference detected. */
		EPropertyBagDiffType DiffType = EPropertyBagDiffType::ValueChanged;

		/** Descriptor on the old side. Null when DiffType == Added. */
		const FPropertyBagPropertyDesc* OldDesc = nullptr;

		/** Descriptor on the new side. Null when DiffType == Removed. */
		const FPropertyBagPropertyDesc* NewDesc = nullptr;

		/**
		 * For MetaDataChanged: the specific metadata key that differs.
		 * For CategoryChanged: NAME_None (key is implicitly "Category").
		 * For all other diff types: NAME_None.
		 */
		FName MetaDataKey = NAME_None;
	};

	/**
	 * Computes per-property differences between two property bags.
	 * Identity is matched by FPropertyBagPropertyDesc::ID so renames are
	 * detected as a single Renamed entry instead of an Added/Removed pair.
	 *
	 * One property may produce multiple entries when more than one dimension
	 * changed (e.g. Renamed + ValueChanged + CategoryChanged are emitted
	 * independently). The caller is responsible for keeping Old and New
	 * alive while iterating the returned descriptor pointers.
	 */
	STRUCTUTILSEDITOR_API void DiffPropertyBags(const FInstancedPropertyBag& Old, const FInstancedPropertyBag& New, TArray<FPropertyBagDiffEntry>& OutDiffs);
}

