// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GraphEditorDragDropAction.h"
#include "IDetailDragDropHandler.h"
#include "StructUtils/PropertyBag.h"

#define UE_API STRUCTUTILSEDITOR_API

class FPropertyBagDetailsDragDropOp;
class IPropertyBagEdGraphDragAndDrop;
struct FHierarchyElementViewModel;

DECLARE_DELEGATE_RetVal_TwoParams(TOptional<EItemDropZone>, FCanAcceptPropertyBagDetailsRowDropOp, TSharedPtr<FPropertyBagDetailsDragDropOp> /*DropOperation*/, EItemDropZone /*DropZone*/);
DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnPropertyBagDetailsRowDropOp, const FPropertyBagPropertyDesc& /*PropertyDescription*/, EItemDropZone /*DropZone*/);
DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnPropertyBagHierarchyElementDropOp, TSharedPtr<FHierarchyElementViewModel> /*DroppedElement*/, EItemDropZone /*DropZone*/);

/** State of the drop, useful for source-is-target validation on details rows. */
enum class EPropertyBagDropState
{
	Invalid,
	Valid,
	SourceIsTarget
};

/**
 * Provides information about the source row (single property or hierarchy element) being dragged.
 * Inherits from FGraphEditorDragDropAction to support dragging to a graph pin/node/panel.
 */
class FPropertyBagDetailsDragDropOp : public FGraphEditorDragDropAction
{
public:
	DRAG_DROP_OPERATOR_TYPE(FPropertyBagDetailsDragDropOp, FGraphEditorDragDropAction);

	struct FDecoration
	{
		explicit FDecoration(const FText& InMessage, const FSlateBrush* InIcon = nullptr, const FLinearColor& InColor = FLinearColor::White);

		const FText Message;
		const FSlateBrush* Icon;
		const FLinearColor IconColor;
	};

	/**
	 * Property Bag Details - Drop Operation Constructor for property elements.
	 * @param InPropertyDesc Property description for the dropped property.
	 */
	UE_API FPropertyBagDetailsDragDropOp(const FPropertyBagPropertyDesc& InPropertyDesc);

	/**
	 * Sets the pop up widget's icon and text.
	 * @param NewDropState The state of the drop operation.
	 * @param OverriddenDecoration An optional overridden decorator for the UI, including Message, Icon, and IconColor
	 */
	UE_API void SetDecoration(EPropertyBagDropState NewDropState, TOptional<FDecoration> OverriddenDecoration = {});

	// ~Begin FGraphEditorDragDropAction interface
	/** Event when the drag and drop operation changes hover targets. */
	UE_API virtual void HoverTargetChanged() override;

	/** Visibility of the cursor decorator icon. */
	UE_API virtual EVisibility GetIconVisible() const override;
	/** Visibility of the cursor decorator error icon. */
	UE_API virtual EVisibility GetErrorIconVisible() const override;

	UE_API virtual FReply DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	UE_API virtual FReply DroppedOnNode(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	UE_API virtual FReply DroppedOnPanel(const TSharedRef<SWidget>& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	// ~End FGraphEditorDragDropAction interface

	/** The property description of the dragged property. May be default-constructed for non-property hierarchy elements. */
	const FPropertyBagPropertyDesc PropertyDesc;
	/** Cached state for the decorator. */
	EPropertyBagDropState CurrentDropState = EPropertyBagDropState::Invalid;

	/** Optional hierarchy element view model for hierarchy-based drag & drop in the details panel. */
	UE_API void SetHierarchyElement(TSharedPtr<FHierarchyElementViewModel> InElement);
	UE_API TSharedPtr<FHierarchyElementViewModel> GetHierarchyElement() const;

private:
	UE_API IPropertyBagEdGraphDragAndDrop* GetPropertyBagEdGraphDragAndDropInterface() const;

	TWeakPtr<FHierarchyElementViewModel> HierarchyElement;
};

/** Handles drag-and-drop (as a target) for a single property's child widget row. */
class FPropertyBagDetailsDragDropHandlerTarget : public IDetailDragDropHandler
{
public:
	FPropertyBagDetailsDragDropHandlerTarget() = default;

	UE_API FPropertyBagDetailsDragDropHandlerTarget(const FCanAcceptPropertyBagDetailsRowDropOp& CanAcceptDragDrop, const FOnPropertyBagDetailsRowDropOp& OnDragDrop);

	/** Creates the drag and drop operation. Disabled for the target. Enabled in subclasses. */
	UE_API virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation() const override;

	UE_API virtual void BindCanAcceptDragDrop(FCanAcceptPropertyBagDetailsRowDropOp&& CanAcceptDragDrop);
	UE_API virtual void BindOnHandleDragDrop(FOnPropertyBagDetailsRowDropOp&& OnDragDrop);

	/** Bind a hierarchy-element-based drop handler. Used when the dropped element may not have a valid PropertyDesc (e.g. categories). */
	UE_API virtual void BindOnHandleHierarchyElementDrop(FOnPropertyBagHierarchyElementDropOp&& OnDrop);

	/** Disable automatic creation of the handle widget for targets. It is overridden in the source below. */
	virtual bool UseHandleWidget() const override
	{
		return false;
	}

protected:
	UE_API virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropSource, EItemDropZone DropZone) const override;
	UE_API virtual bool AcceptDrop(const FDragDropEvent& DragDropSource, EItemDropZone DropZone) const override;

private:
	FCanAcceptPropertyBagDetailsRowDropOp CanAcceptDetailsRowDropOp;
	FOnPropertyBagDetailsRowDropOp OnHandleDetailsRowDropOp;
	FOnPropertyBagHierarchyElementDropOp OnHandleHierarchyElementDropOp;
};

/** Handles drag-and-drop (as a source or target) for a single property's child widget row. */
class FPropertyBagDetailsDragDropHandler : public FPropertyBagDetailsDragDropHandlerTarget
{
public:
	UE_API FPropertyBagDetailsDragDropHandler(const FPropertyBagPropertyDesc& InPropertyDesc);

	/** Set an optional hierarchy element view model to attach to drag ops created by this handler. */
	void SetHierarchyElement(TSharedPtr<FHierarchyElementViewModel> InElement) { HierarchyElement = InElement; }

protected:
	UE_API virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation() const override;

	/** Enable the automatic creation of the handle (grip) widget for the source. */
	virtual bool UseHandleWidget() const override { return true; }

private:
	/** The current child row's property bag property description. */
	const FPropertyBagPropertyDesc PropertyDesc;

	/** Optional hierarchy element for hierarchy-based drag & drop. */
	TWeakPtr<FHierarchyElementViewModel> HierarchyElement;
};

#undef UE_API
