// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructViewerFilter.h"
#include "StructViewerModule.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API STRUCTUTILSEDITOR_API

class IPropertyHandle;
class IPropertyUtilities;
class IAssetReferenceFilter;
class SComboButton;

/**
 * Filter used by the instanced struct struct picker.
 */
class FInstancedStructFilter : public IStructViewerFilter
{
public:
	/** The base struct for the property that classes must be a child-of. */
	TWeakObjectPtr<const UScriptStruct> BaseStruct = nullptr;

	/** The array of allowed structs */
	TArray<TSoftObjectPtr<const UScriptStruct>> AllowedStructs;

	/** The array of disallowed structs */
	TArray<TSoftObjectPtr<const UScriptStruct>> DisallowedStructs;

	// A flag controlling whether we allow UserDefinedStructs
	bool bAllowUserDefinedStructs = false;

	// A flag controlling whether we allow to select the BaseStruct
	bool bAllowBaseStruct = true;

	// A flag controlling whether we allow child structs of AllowedStructs / DisallowedStructs
	bool bAllowChildStructs = true;

	UE_API virtual bool IsStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const UScriptStruct* InStruct, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override;
	UE_API virtual bool IsUnloadedStructAllowed(const FStructViewerInitializationOptions& InInitOptions, const FSoftObjectPath& InStructPath, TSharedRef<FStructViewerFilterFuncs> InFilterFuncs) override;

	// Optional filter to prevent selection of some structs e.g. ones in a plugin that is inaccessible from the object being edited
	TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter;
};

class SInstancedStructPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SInstancedStructPicker) { }
		SLATE_EVENT(FOnStructPicked, OnStructPicked)
		SLATE_ARGUMENT(TArray<TSoftObjectPtr<const UScriptStruct>>, AllowedStructs)
		SLATE_ARGUMENT(TArray<TSoftObjectPtr<const UScriptStruct>>, DisallowedStructs)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedPtr<IPropertyHandle> InStructProperty, TSharedPtr<IPropertyUtilities> InPropertyUtils);

	const UScriptStruct* GetBaseScriptStruct() const;

	bool CanChangeStructType() const
	{
		return bCanChangeStructType;
	}

	FOnStructPicked OnStructPicked;

private:
	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<IPropertyHandle> StructProperty;
	TSharedPtr<IPropertyUtilities> PropUtils;

	/** Can we change the struct type at all? (controlled by the "StructTypeConst" meta-data) */
	bool bCanChangeStructType = true;

	/** The array of allowed structs (additionally to the meta-data) */
	TArray<TSoftObjectPtr<const UScriptStruct>> AllowedStructs;

	/** The array of disallowed structs (additionally to the meta-data) */
	TArray<TSoftObjectPtr<const UScriptStruct>> DisallowedStructs;

	UE_API FText GetDisplayValueString() const;
	UE_API FText GetTooltipText() const;
	UE_API const FSlateBrush* GetDisplayValueIcon() const;
	UE_API TSharedRef<SWidget> GenerateStructPicker();
	UE_API void StructPicked(const UScriptStruct* InStruct);
};

#undef UE_API
