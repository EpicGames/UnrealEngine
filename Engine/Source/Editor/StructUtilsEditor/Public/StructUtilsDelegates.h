// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class UUserDefinedStruct;

namespace UE::StructUtils::Delegates
{
#if WITH_EDITOR
	/** Called after the FInstancedStructs has been reinstantiated. E.g. safe to update UI. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUserDefinedStructReinstanced, const UUserDefinedStruct& /*UserDefinedStruct*/);
	extern STRUCTUTILSEDITOR_API FOnUserDefinedStructReinstanced OnUserDefinedStructReinstanced;
#endif
} // UE::StructUtils::Delegates
