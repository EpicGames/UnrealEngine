// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "Misc/NotNull.h"
#include "Modules/ModuleInterface.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"

#define UE_API STRUCTUTILSEDITOR_API

class FPropertyBagHierarchyViewModelPropertyEditorPolicy;
class IStructUtilsEditor;
struct FGraphPanelNodeFactory;
class UUserDefinedStruct;
class UPropertyBagHierarchyRoot;
class UPropertyBagHierarchyViewModel;
class IPropertyUtilities;
class FPropertyPath;

/**
 * Shared owner for a UPropertyBagHierarchyViewModel.
 *
 * Holds the view model strongly. When the last TSharedPtr to this owner is released,
 * the destructor finalizes the view model and marks it as garbage, stopping its
 * editor tick deterministically and releasing it to the next GC pass.
 *
 * Consumers acquire one via FStructUtilsEditorModule::AcquireHierarchyViewModel and
 * hold the returned TSharedPtr for as long as the view model is needed. Direct
 * construction is not supported.
 */
struct FPropertyBagHierarchyViewModelOwner
{
	UE_API explicit FPropertyBagHierarchyViewModelOwner(UPropertyBagHierarchyViewModel* InViewModel);
	UE_API ~FPropertyBagHierarchyViewModelOwner();

	FPropertyBagHierarchyViewModelOwner(const FPropertyBagHierarchyViewModelOwner&) = delete;
	FPropertyBagHierarchyViewModelOwner& operator=(const FPropertyBagHierarchyViewModelOwner&) = delete;
	FPropertyBagHierarchyViewModelOwner(FPropertyBagHierarchyViewModelOwner&&) = delete;
	FPropertyBagHierarchyViewModelOwner& operator=(FPropertyBagHierarchyViewModelOwner&&) = delete;

	UPropertyBagHierarchyViewModel* Get() const { return ViewModel.Get(); }

private:
	TStrongObjectPtr<UPropertyBagHierarchyViewModel> ViewModel;
};

/**
* The public interface to this module
*/
class FStructUtilsEditorModule : public IModuleInterface, public FStructureEditorUtils::INotifyOnStructChanged
{
public:
	// Begin IModuleInterface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	/**
	 * Returns a shared owner of the hierarchy view model for the given root, creating one if needed.
	 *
	 * Hold the returned TSharedRef for as long as the view model is needed. When the last consumer
	 * drops it, the view model is finalized and marked as garbage, immediately stopping its editor
	 * tick. Subsequent calls for the same root will produce a fresh view model.
	 */
	UE_API TSharedRef<FPropertyBagHierarchyViewModelOwner> AcquireHierarchyViewModel(
		UPropertyBagHierarchyRoot* InHierarchyRoot,
		UObject* InOwningObject,
		TSharedRef<FPropertyPath> InPropertyPath,
		TSharedRef<IPropertyUtilities> PropertyUtilities);

protected:

	// INotifyOnStructChanged
	UE_API virtual void PreChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info) override;
	UE_API virtual void PostChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info) override;
	// ~INotifyOnStructChanged

private:
	void OnPostGarbageCollect();

	/** View model owner lookup by hierarchy root. Weak both ways. Strong ownership lives in TSharedPtr consumers. */
	TMap<TWeakObjectPtr<UPropertyBagHierarchyRoot>, TWeakPtr<FPropertyBagHierarchyViewModelOwner>> HierarchyViewModelMap;

	TUniquePtr<FPropertyBagHierarchyViewModelPropertyEditorPolicy> HierarchyViewModelPropertyEditorPolicy;

	FDelegateHandle PostGarbageCollectHandle;
};

#undef UE_API
