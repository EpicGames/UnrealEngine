// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/ValueOrError.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"

#define UE_API STRUCTUTILSEDITOR_API

class UUserDefinedStruct;
struct FInstancedPropertyBag;
struct FPropertyBagPropertyDesc;

namespace UE::StructUtils
{
	struct FCreateUserDefinedStructArgs
	{
		/** The default UserDefinedStruct flags. */
		EObjectFlags UserDefinedStructFlags = RF_Transactional | RF_Public;
		/** Add the BlueprintType metadata to the struct. */
		bool bIsBlueprintType = true;
		/**
		 * The UserDefinedStruct editor appends the property ID to the property name.
		 * Whether the newly created User Defined Struct should reproduce the same behavior.
		 */
		bool bAddPropertyIDToPropertyName = false;
	};


	/**
	 * @return a new user defined struct created from a property bag.
	 * Fails if the name is already used by another asset
	 *	or if the asset could not be created
	 *	or the FInstancedPropertyBag contains type that are not supported by UserDefinedStruct.
	 */
	UE_API TValueOrError<UUserDefinedStruct*, void> CreateUserDefinedStructFromDescs(TNotNull<UObject*> UserDefinedStructOuter, const FInstancedPropertyBag& PropertyBag, FName UserDefinedStructName, const FCreateUserDefinedStructArgs& Args);

	/**
	 * Update, if needed, an existing user defined struct to have the exact same properties and default value as the property bag.
	 * @return true if an update occurs.
	 * Fails if FInstancedPropertyBag contains type that are not supported by UserDefinedStruct.
	 */
	UE_API TValueOrError<bool, void> UpdateUserDefinedStructFromDescs(TNotNull<UUserDefinedStruct*> Struct, const FInstancedPropertyBag& PropertyBag);
}

#undef UE_API
