// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

struct FPropertyBagPropertyDesc;

// Metadata used by StructUtils
namespace UE::StructUtils::Metadata
{
	// Metadata usable in UPROPERTY for customizing the behavior when displaying the property in a property panel or graph node

	/// FInstancedPropertyBag
	extern STRUCTUTILSEDITOR_API const FLazyName HideInDetailPanelsName;
	extern STRUCTUTILSEDITOR_API const FLazyName ShowOnlyInnerPropertiesName;

	/// [Bag Property Metadata] FixedLayout: Indicates that the instanced property bag has a fixed layout and it is not possible to add/remove properties.
	extern STRUCTUTILSEDITOR_API const FLazyName FixedLayoutName;

	/// [Bag Property Metadata] DefaultType: Default property type when adding a new Property. Should be taken from enum EPropertyBagPropertyType
	extern STRUCTUTILSEDITOR_API const FLazyName DefaultTypeName;

	/// [Bag Property Metadata] AllowContainers: By default it is always allowed to create containers (Array, Set) for properties. Use AllowContainers=false to disable container support.
	extern STRUCTUTILSEDITOR_API const FLazyName AllowContainersName;

	/// [Bag Property Metadata] PropertyBagSchemaClass: Allow to specify the class for the PropertyBagSchema. Useful to specify which containers are available. If unset or invalid, fallback on UPropertyBagSchema.
	extern STRUCTUTILSEDITOR_API const FLazyName PropertyBagSchemaClassName;

	UE_DEPRECATED(5.6, "Remains for backwards compatibility. Please see 'AllowContainersName'.")
	extern STRUCTUTILSEDITOR_API const FLazyName AllowArraysName;

	/// [Bag Property Metadata] IsPinTypeAccepted: Name of a UFunction with signature bool(FEdGraphPinType). Returns false if the type should be discarded.
	extern STRUCTUTILSEDITOR_API const FLazyName IsPinTypeAcceptedName;

	/// [Bag Property Metadata] CanRemoveProperty: Name of a UFunction with signature bool(FGuid, FName). ID and name of the property that will be removed. Returns false if the property should not be removed.
	extern STRUCTUTILSEDITOR_API const FLazyName CanRemovePropertyName;

	/// [Bag Property Metadata] ChildRowFeatures: A list of UI features available to the PropertyBag properties when displayed in the details view child rows.
	extern STRUCTUTILSEDITOR_API const FLazyName ChildRowFeaturesName;

	/// [Bag Property Metadata] ExtensionClass: Path of a class inheriting from UPropertyBagDetailsExtension
	extern STRUCTUTILSEDITOR_API const FLazyName ExtensionClass;

	/// Common property metadata specifiers
	/// [Property Metadata] EnableCategoriesName: Required to enable the use of categories in general. Per property.
	extern STRUCTUTILSEDITOR_API const FLazyName EnableCategoriesName;
	/// [Property Metadata] CategoryName: The name of the category (grouping of properties).
	extern STRUCTUTILSEDITOR_API const FLazyName CategoryName;


	/// Property metadata specifiers for FInstancedStruct/TInstancedStruct
	/// [PropertyMetadata] BaseStruct: Type name of the base class for the instanced struct
	extern STRUCTUTILSEDITOR_API const FLazyName BaseStructName;

	/// [PropertyMetadata] BaseStructFunction: Name of a UFunction with signature UScriptStruct*(UE::StructUtils::FInstancedStructBaseStructQueryParams). Returns the type of the base struct for the instanced struct.
	/// (cf. InstancedStructBaseStructQueryParams.h for an example of function.)
	/// @note When return value is invalid, BaseStruct will be used if any.
	extern STRUCTUTILSEDITOR_API const FLazyName BaseStructFunctionName;

	/// [PropertyMetadata] StructTypeConst: Determine whether the type of the instanced struct can be changed
	extern STRUCTUTILSEDITOR_API const FLazyName StructTypeConstName;

	/// [PropertyMetadata] HideStructType: Determine whether the type of the instanced struct is visible
	extern STRUCTUTILSEDITOR_API const FLazyName HideStructTypeName;

	/** If organizing by category is enabled for this property. */
	STRUCTUTILSEDITOR_API bool AreCategoriesEnabled(const FPropertyBagPropertyDesc& Desc);
	/** Enable category organization for this property. */
	STRUCTUTILSEDITOR_API void EnableCategories(FPropertyBagPropertyDesc& OutDesc);
	/** Disable category organization for this property. */
	STRUCTUTILSEDITOR_API void DisableCategories(FPropertyBagPropertyDesc& OutDesc);
	/** Set the category for this property. */
	STRUCTUTILSEDITOR_API void SetCategory(FPropertyBagPropertyDesc& OutDesc, const FString& InGroupLabel, bool bAutoEnableCategories = true);
	/** Remove the category for this property. */
	STRUCTUTILSEDITOR_API void RemoveCategory(FPropertyBagPropertyDesc& OutDesc, bool bAutoDisableCategories = true);
	/** Get the category for this property. */
	STRUCTUTILSEDITOR_API FString GetCategory(const FPropertyBagPropertyDesc& InDesc);

	namespace Specifiers
	{
		extern STRUCTUTILSEDITOR_API const FLazyName ToolTipName;
		extern STRUCTUTILSEDITOR_API const FLazyName ClampMinName;
		extern STRUCTUTILSEDITOR_API const FLazyName ClampMaxName;
		extern STRUCTUTILSEDITOR_API const FLazyName UIMinName;
		extern STRUCTUTILSEDITOR_API const FLazyName UIMaxName;
		extern STRUCTUTILSEDITOR_API const FLazyName UnitsName;
		extern STRUCTUTILSEDITOR_API const FLazyName ForceUnitsName;

		STRUCTUTILSEDITOR_API bool SupportsClamping(const FPropertyBagPropertyDesc& InDesc);
		STRUCTUTILSEDITOR_API bool SupportsSettingUnits(const FPropertyBagPropertyDesc& InDesc);
	}
}
