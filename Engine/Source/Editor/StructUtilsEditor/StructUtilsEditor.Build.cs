// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class StructUtilsEditor : ModuleRules
	{
		public StructUtilsEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
			bAllowUETypesInNamespaces = true;

			PublicIncludePaths.AddRange(
				new string[] {
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"AssetTools",
					"UnrealEd",
					"Slate",
					"SlateCore",
					"PropertyEditor",
					"AIModule",
					"DetailCustomizations",
					"ComponentVisualizers",
					"DataHierarchyEditor"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ApplicationCore",
					"RenderCore",
					"GraphEditor",
					"KismetWidgets",
					"PropertyEditor",
					"AIGraph",
					"ToolMenus",
					"BlueprintGraph",
					"KismetCompiler"
				}
			);
		}

	}
}
